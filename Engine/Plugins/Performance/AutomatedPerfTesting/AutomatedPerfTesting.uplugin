{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "0.1",
	"FriendlyName": "AutomatedPerfTesting",
	"Description": "This plugin provides Gauntlet Test Controllers to facilitate automatic performance testing.",
	"Category": "Testing",
	"CreatedBy": "",
	"CreatedByURL": "",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"CanContainContent": true,
	"IsBetaVersion": false,
	"IsExperimentalVersion": true,
	"Installed": false,
	"Modules": [
		{
			"Name": "AutomatedPerfTesting",
			"Type": "Runtime",
			"LoadingPhase": "Default"
		}
	],
	"Plugins": [
		{
			"Name": "Gauntlet",
			"Enabled": true
		},
		{
			"Name": "ProjectLauncher",
			"Enabled": true,
			"SupportedTargetPlatforms": [
				"Win64",
				"Linux",
				"Mac"
			]
		}
	]
}