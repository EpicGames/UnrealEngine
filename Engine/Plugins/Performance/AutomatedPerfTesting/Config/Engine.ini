[SystemSettings]
; if you need to override this per platform, this needs to be done in Engine/Platforms/<Platform>/Plugins/Performance/AutomatedPerfTesting/Config/<Platform>Engine.ini
APT.DynamicRes.LockedScreenPercentage=60
