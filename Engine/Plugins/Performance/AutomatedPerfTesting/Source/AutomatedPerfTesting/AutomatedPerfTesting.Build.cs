// Copyright Epic Games, Inc. All Rights Reserved.

using System.Linq;
using UnrealBuildTool;

public class AutomatedPerfTesting : ModuleRules
{
	UnrealTargetPlatform[] SupportedProjectLauncherPlatforms => [
		UnrealTargetPlatform.Win64,
		UnrealTargetPlatform.Linux,
		UnrealTargetPlatform.Mac
	];

	public AutomatedPerfTesting(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;
		
		PublicIncludePaths.AddRange(
			new string[] {
				// ... add public include paths required here ...
			}
			);
				
		
		PrivateIncludePaths.AddRange(
			new string[] {
				// ... add other private include paths required here ...
			}
			);
			
		
		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"Core", "Gauntlet"
				// ... add other public dependencies that you statically link with here ...
			}
			);

		if(Target.bCompileAgainstEditor && SupportedProjectLauncherPlatforms.Contains(Target.Platform))
		{
			PublicDependencyModuleNames.AddRange(
			[
				"ProjectLauncher", 			
				"Projects",
				"UnrealEd"
			]);
		}

		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"CoreUObject",
				"Engine",
				"Slate",
				"SlateCore", 
				"LevelSequence", 
				"MovieScene", 
				"DeveloperSettings",        
				"Json",
				"JsonUtilities"
				// ... add private dependencies that you statically link with here ...	
			}
			);
		
		DynamicallyLoadedModuleNames.AddRange(
			new string[]
			{
				// ... add any modules that your module loads dynamically here ...
			}
			);
	}
}
