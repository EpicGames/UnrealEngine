// Copyright Epic Games, Inc. All Rights Reserved.

#include "AutomatedMaterialPerfTest.h"
#include "TimerManager.h"
#include "Kismet/GameplayStatics.h"
#include "AutomatedPerfTesting.h"
#include "Camera/CameraActor.h"
#include "Camera/CameraComponent.h"
#include "Engine/StaticMeshActor.h"
#include "Engine/StaticMesh.h"
#include "Engine/Player.h"
#include "GameFramework/Pawn.h"
#include "Logging/StructuredLog.h"
#include "Materials/MaterialInterface.h"
#include "ProfilingDebugging/CsvProfiler.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AutomatedMaterialPerfTest)

UAutomatedMaterialPerfTestProjectSettings::UAutomatedMaterialPerfTestProjectSettings(const FObjectInitializer& Initializer)
	: Super(Initializer)
	, bCaptureScreenshots(true)
	, WarmUpTime(5.0)
	, SoakTime(5.0)
	, CooldownTime(1.0)
	, MaterialPerformanceTestMap(FSoftObjectPath("/AutomatedPerfTesting/Tests/Materials/AutomatedMaterialPerfTestDefaultMap.AutomatedMaterialPerfTestDefaultMap"))
	, CameraProjectionMode(ECameraProjectionMode::Type::Orthographic)
	, PlateDistanceFromCamera(512.0)
	, MaterialPlate(FSoftObjectPath("/AutomatedPerfTesting/Tests/Materials/SM_AutomatedMaterialPerfTestDefaultPlate.SM_AutomatedMaterialPerfTestDefaultPlate"))
{
}

void UAutomatedMaterialPerfTest::SetupTest()
{
	// load up into the map defined in project settings
	if(GetCurrentMap() == Settings->MaterialPerformanceTestMap.GetAssetName())
	{
		Super::SetupTest();

		UE_LOGFMT(LogAutomatedPerfTest, Log, "UAutomatedMaterialPerfTest::SetupTest");
		// make sure the world exists, then create a sequence player
		if(UWorld* const World = GetWorld())
		{
			SetupProfiling();

			// Also load and verify the material plate mesh
			UE_LOGF(LogAutomatedPerfTest, Log, "Loading material plate mesh: %ls", *Settings->MaterialPlate.ToString())
			UStaticMesh* LoadedMaterialPlateMesh = LoadObject<UStaticMesh>(NULL, *Settings->MaterialPlate.ToString(), NULL, LOAD_None, NULL);
			if (LoadedMaterialPlateMesh == nullptr)
			{
				UE_LOGF(LogAutomatedPerfTest, Error, "Failed to load the setting specified MaterialPlate '%ls'. Exiting...", 
					*Settings->MaterialPlate.ToString());

				EndTestFailure();
				return;
			}

			// reset the material index;
			CurrentMaterialIndex = -1;
			
			// hide the pawn so it doesn't interfere with screenshots
			GetFirstPlayerController()->GetPawn()->SetHidden(true);
			
			// then spawn the camera into the world and set it up 
			Camera = GetWorld()->SpawnActor<ACameraActor>();
			if (Camera == nullptr)
			{
				UE_LOGFMT(LogAutomatedPerfTest, Error, "Failed to spawn ACameraActor. Exiting...");

				EndTestFailure();
				return;
			}

			Camera->GetCameraComponent()->SetProjectionMode(Settings->CameraProjectionMode);
			Camera->GetCameraComponent()->SetOrthoWidth(Settings->PlateDistanceFromCamera);

			// and spawn the material plate into the world, and move it PlateDistanceFromCamera away down the X axis
			MaterialPlate = GetWorld()->SpawnActor<AStaticMeshActor>();
			MaterialPlate->SetMobility(EComponentMobility::Type::Movable);
			MaterialPlate->GetStaticMeshComponent()->SetStaticMesh(LoadedMaterialPlateMesh);
			MaterialPlate->SetActorLocation(FVector(Settings->PlateDistanceFromCamera, 0.0, 0.0));

			// Ensure the loaded plate width is a viable value for the division.
			float PlateHalfWidth = LoadedMaterialPlateMesh->GetBounds().BoxExtent.Y;
			constexpr float MinPlateHalfExtentsCm = 1.0f;
			if (PlateHalfWidth < MinPlateHalfExtentsCm)
			{
				UE_LOGF(LogAutomatedPerfTest, Warning, "Setting specified MaterialPlate '%ls' extent '%f' are less than the minimum '%f'. Using '%f'.", 
					*Settings->MaterialPlate.ToString(), PlateHalfWidth, MinPlateHalfExtentsCm, MinPlateHalfExtentsCm);
				PlateHalfWidth = MinPlateHalfExtentsCm;
			}

			float Scale = Settings->PlateDistanceFromCamera / PlateHalfWidth;

			UE_LOGF(LogAutomatedPerfTest, Verbose, "SizeY = %f, Scale = %f", LoadedMaterialPlateMesh->GetBounds().BoxExtent.Y, Scale);
			
			MaterialPlate->SetActorScale3D(FVector(1.0, Scale, Scale));
			
			GetFirstPlayerController()->SetViewTarget(Camera);
			
        	// delay for WarmUpDelay, and call RunTest
        	FTimerHandle UnusedHandle;
        	GetWorld()->GetTimerManager().SetTimer(UnusedHandle, this, &UAutomatedMaterialPerfTest::RunTest, 1.0, false, Settings->WarmUpTime);
		}
		// if we have an invalid world, we can't run the test, so we should bail out
		else
		{
			UE_LOGFMT(LogAutomatedPerfTest, Error, "Invalid World when starting UAutomatedMaterialPerfTest, exiting...");
			EndTestFailure();
		}
	}
	else
	{
		UE_LOGF(LogAutomatedPerfTest, Log, "Current Map Name %ls is not the expected %ls, loading the material performance test map", *GetCurrentMap(), *Settings->MaterialPerformanceTestMap.GetAssetName())
		OpenMaterialPerformanceTestMap();
	}
}

void UAutomatedMaterialPerfTest::RunTest()
{
	Super::RunTest();

	// The overall profiling region starts here, but each material gets its own sub region for easy delineation. 
	MarkProfilingStart(); 

	UE_LOGFMT(LogAutomatedPerfTest, Log, "UAutomatedMaterialPerfTest::RunTest");
	
	FTimerHandle UnusedHandle;
	GetWorld()->GetTimerManager().SetTimer(UnusedHandle, this, &UAutomatedMaterialPerfTest::SetUpNextMaterial, 1.0, false, Settings->WarmUpTime);
}

FString UAutomatedMaterialPerfTest::GetPerfTestTypeID() const
{
	return TEXT("Materials");
}

void UAutomatedMaterialPerfTest::GatherTestMetadata(TArray<TPair<FString, FString>>& OutMetadata) const
{
	Super::GatherTestMetadata(OutMetadata);

	OutMetadata.Emplace(TEXT("Material"), GetCurrentMaterialRegionName());
}

void UAutomatedMaterialPerfTest::SetUpNextMaterial()
{
	CurrentMaterialIndex += 1;
	
	if(CurrentMaterialIndex >= Settings->MaterialsToTest.Num())
	{
		UE_LOGFMT(LogAutomatedPerfTest, Log, "No more materials left to test, moving to teardown.");
		TeardownTest();
		return;
	}
	
	// load the next material
	FSoftObjectPath MaterialSoftObjectPath = Settings->MaterialsToTest[CurrentMaterialIndex];
	
	UE_LOGF(LogAutomatedPerfTest, Log, "Loading material: %ls", *MaterialSoftObjectPath.ToString())
	CurrentMaterial = LoadObject<UMaterialInterface>(NULL, *MaterialSoftObjectPath.ToString(), NULL, LOAD_None, NULL);
	check(CurrentMaterial);

	UE_LOGF(LogAutomatedPerfTest, Log, "Applying material: %ls", *CurrentMaterial->GetName());
	
	MaterialPlate->GetStaticMeshComponent()->SetMaterial(0, CurrentMaterial);
	
	FTimerHandle UnusedHandle;
	GetWorld()->GetTimerManager().SetTimer(UnusedHandle, this, &UAutomatedMaterialPerfTest::EvaluateMaterial, 1.0, false, Settings->WarmUpTime);
}

void UAutomatedMaterialPerfTest::EvaluateMaterial()
{
	MarkMaterialStart();

	FTimerHandle UnusedHandle;
	GetWorld()->GetTimerManager().SetTimer(UnusedHandle, this, &UAutomatedMaterialPerfTest::FinishMaterialEvaluation, 1.0, false, Settings->SoakTime);
}

void UAutomatedMaterialPerfTest::FinishMaterialEvaluation()
{
	MarkMaterialEnd();
	
	if(Settings->bCaptureScreenshots)
	{
		FTimerHandle UnusedHandle;
		GetWorld()->GetTimerManager().SetTimer(UnusedHandle, this, &UAutomatedMaterialPerfTest::ScreenshotMaterial, 1.0, false, Settings->CooldownTime);		
	}
	else
	{
		FTimerHandle UnusedHandle;
		GetWorld()->GetTimerManager().SetTimer(UnusedHandle, this, &UAutomatedMaterialPerfTest::SetUpNextMaterial, 1.0, false, Settings->CooldownTime);
	}
}

void UAutomatedMaterialPerfTest::ScreenshotMaterial()
{
	TakeScreenshot(GetCurrentMaterialRegionName());

	// start a timer to trigger the disk screenshot, since trace screenshots and disk screenshots can't happen in the same frame
	FTimerHandle UnusedHandle;
	GetWorld()->GetTimerManager().SetTimer(UnusedHandle, this, &UAutomatedMaterialPerfTest::SetUpNextMaterial, 1.0, false, .1f);
}

FString UAutomatedMaterialPerfTest::GetCurrentMaterialRegionName() const
{
	return CurrentMaterial->GetName();
}

void UAutomatedMaterialPerfTest::MarkMaterialStart()
{
	// safety check on the current material
	if(CurrentMaterial)
	{
		if(RequestsInsightsTrace())
		{
			TRACE_BEGIN_REGION(*GetCurrentMaterialRegionName());
		}
#if CSV_PROFILER
		if(RequestsCSVProfiler())
		{
			FString CSVFilename = GetTestID() + "_" + GetCurrentMaterialRegionName();
			TryStartCSVProfiler(CSVFilename);
		}
#endif
	}
}

void UAutomatedMaterialPerfTest::MarkMaterialEnd()
{
	// safety check on the current material
	if(CurrentMaterial)
	{
		if(RequestsInsightsTrace())
		{
			TRACE_END_REGION(*GetCurrentMaterialRegionName());
		}
#if CSV_PROFILER
		if(RequestsCSVProfiler())
		{
			TryStopCSVProfiler();
		}
#endif
	}
}

void UAutomatedMaterialPerfTest::OnInit()
{
	Super::OnInit();
	
	UE_LOGFMT(LogAutomatedPerfTest, Log, "UAutomatedMaterialPerfTest::OnInit");

	Settings = GetDefault<UAutomatedMaterialPerfTestProjectSettings>();

	// early out if there aren't actually any materials
	if (Settings->MaterialsToTest.IsEmpty())
	{
		UE_LOGFMT(LogAutomatedPerfTest, Error, "No materials defined in the project's Automated Perf Test | Materials settings. Exiting test early.");
		EndTestFailure();
	}
}

void UAutomatedMaterialPerfTest::UnbindAllDelegates()
{
	Super::UnbindAllDelegates();

	if (UWorld* const World = GetWorld())
	{
		World->GetTimerManager().ClearAllTimersForObject(this);
	}
}

void UAutomatedMaterialPerfTest::TeardownTest(bool bExitAfterTeardown)
{
	UE_LOGFMT(LogAutomatedPerfTest, Log, "AutomatedMaterialPerfTest::TeardownTest");
	MarkProfilingEnd();
	TeardownProfiling();
	Super::TeardownTest(bExitAfterTeardown);
}

void UAutomatedMaterialPerfTest::OpenMaterialPerformanceTestMap() const
{
	// no need to prepend this with a ? since OpenLevel handles that part for us
	FString OptionsString;
	if(!Settings->GameModeOverride.IsEmpty())
	{
		UE_LOGF(LogAutomatedPerfTest, Log, "Game Mode overridden to %ls", *Settings->GameModeOverride)
		OptionsString += "game=" + Settings->GameModeOverride;
	}
	
	UE_LOGF(LogAutomatedPerfTest, Log, "Opening map %ls%ls", *Settings->MaterialPerformanceTestMap.GetAssetName(), *OptionsString);
	UGameplayStatics::OpenLevel(AutomatedPerfTest::FindCurrentWorld(), *Settings->MaterialPerformanceTestMap.GetAssetName(), true, OptionsString);
}
