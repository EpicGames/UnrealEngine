// Copyright Epic Games, Inc. All Rights Reserved.

#include "AutomatedPerfTestControllerBase.h"

#include "AutomatedPerfTesting.h"
#include "DeviceProfiles/DeviceProfileManager.h"
#include "GameFramework/GameModeBase.h"
#include "Engine/LocalPlayer.h"
#include "AutomatedPerfTestInterface.h"
#include "AutomatedPerfTestProjectSettings.h"
#include "Misc/App.h"
#include "Misc/CommandLine.h"
#include "ProfilingDebugging/TraceAuxiliary.h"
#include "TimerManager.h"
#include "ProfilingDebugging/CsvProfiler.h"
#include "Async/TaskGraphInterfaces.h"
#include "VideoRecordingSystem.h"
#include "PlatformFeatures.h"
#include "Engine/Engine.h"
#include "Misc/Paths.h"
#include "ProfilingDebugging/TraceScreenshot.h"
#include "UnrealClient.h"
#include "UnrealEngine.h"
#include "GauntletModule.h"
#include "Logging/StructuredLog.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AutomatedPerfTestControllerBase)

DEFINE_LOG_CATEGORY(LogAutomatedPerfTest)
CSV_DEFINE_CATEGORY(AutomatedPerfTest, true);

static float GAPTDynamicResLockedScreenPercentage = 100.f;
static FAutoConsoleVariableRef CVarDynamicResLockedScreenPercentage(
	TEXT("APT.DynamicRes.LockedScreenPercentage"),
	GAPTDynamicResLockedScreenPercentage,
	TEXT("Target resolution percentage, configurable per platform. Use -AutomatedPerfTest.LockDynamicRes to force the resolution to this"),
	ECVF_Default
);

void UAutomatedPerfTestControllerBase::OnPreWorldInitializeInternal(UWorld* World, const UWorld::InitializationValues IVS)
{
	OnPreWorldInitialize(World);

	if (RequestsLockedDynRes())
	{
		IConsoleVariable* CVarTestScreenPercentage = IConsoleManager::Get().FindConsoleVariable(TEXT("r.dynamicres.testscreenpercentage"));
		if (ensure(CVarTestScreenPercentage))
		{
			if (CVarTestScreenPercentage->GetFloat() != GAPTDynamicResLockedScreenPercentage)
			{
				UE_LOGF(LogAutomatedPerfTest, Display, "Locking screen percentage to %.2f", GAPTDynamicResLockedScreenPercentage);
				CVarTestScreenPercentage->Set(GAPTDynamicResLockedScreenPercentage);
			}
		}
	}
}

void UAutomatedPerfTestControllerBase::OnPreWorldInitialize(UWorld* World)
{
	check(World);
	World->OnWorldBeginPlay.AddUObject(this, &ThisClass::OnWorldBeginPlay);
}

void UAutomatedPerfTestControllerBase::OnWorldBeginPlay()
{
	UE_LOGFMT(LogAutomatedPerfTest, Log, "OnWorldBeginPlay");
	SetupTest();
}

UAutomatedPerfTestControllerBase::UAutomatedPerfTestControllerBase(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, TraceChannels("default,screenshot,stats")
	, bRequestsFPSChart(false)
	, bRequestsInsightsTrace(false)
	, bRequestsCSVProfiler(false)
	, bRequestsVideoCapture(false)
	, bRequestsLockedDynRes(false)
	, InsightsRegionID(0)
	, ArtifactOutputPath()
{
	// cache this off once, so that it's consistent throughout a session
	TestDatetime = FDateTime::Now().ToString(TEXT("%Y%m%d-%H%M%S"));
}

FString UAutomatedPerfTestControllerBase::GetTestName() const
{
	return TestID;
}

FString UAutomatedPerfTestControllerBase::GetTestID() const
{
	const TArray<FString> TestCaseIDElements = {FApp::GetBuildVersion(),
										  FPlatformProperties::PlatformName(),
										  TestDatetime,
								          UDeviceProfileManager::Get().GetActiveDeviceProfileName(),
								          GetTestName(),
										  GetPerfTestTypeID()};

	// TODO make this format definable in project settings, similar to how MovieRenderQueue does it for renders
	// construct a unique ID of the form BuildVersion_PlatformName_YYYYMMDD-HHMMSS_DeviceProfile_TestName_TestTypeID
	FString TestCaseID = FString::Join(TestCaseIDElements, TEXT("_"));

	return TestCaseID;
}

FString UAutomatedPerfTestControllerBase::GetOverallRegionName()
{
	return GetTestID() + "_" + "Overall";
}

FString UAutomatedPerfTestControllerBase::GetTraceChannels()
{
	return TraceChannels;
}

bool UAutomatedPerfTestControllerBase::RequestsInsightsTrace() const
{
	return bRequestsInsightsTrace;
}

bool UAutomatedPerfTestControllerBase::RequestsCSVProfiler() const
{
	return bRequestsCSVProfiler;
}

bool UAutomatedPerfTestControllerBase::RequestsFPSChart() const
{
	return bRequestsFPSChart;
}

FString UAutomatedPerfTestControllerBase::GetPerfTestTypeID() const
{
	return TEXT("");
}

bool UAutomatedPerfTestControllerBase::RequestsVideoCapture() const
{
	return bRequestsVideoCapture;
}

bool UAutomatedPerfTestControllerBase::RequestsLockedDynRes() const
{
	return bRequestsLockedDynRes;
}

bool UAutomatedPerfTestControllerBase::TryStartInsightsTrace()
{
	const FString TraceFileName = GetTestID() + ".utrace";
	UE_LOGF(LogAutomatedPerfTest, Log, "Attempting to start insights trace to file %ls with channels %ls", *TraceFileName, *GetTraceChannels());
	return FTraceAuxiliary::Start(FTraceAuxiliary::EConnectionType::File, *TraceFileName, *GetTraceChannels());
}

bool UAutomatedPerfTestControllerBase::TryStopInsightsTrace()
{
	if(FTraceAuxiliary::IsConnected())
	{
		return FTraceAuxiliary::Stop();
	}
	return false;
}

void UAutomatedPerfTestControllerBase::GatherTestMetadata(TArray<TPair<FString, FString>>& OutMetadata) const
{
	OutMetadata.Emplace(TEXT("TestID"), TestID);
	OutMetadata.Emplace(TEXT("Datetime"), TestDatetime);
	OutMetadata.Emplace(TEXT("ResX"), FString::FromInt(GSystemResolution.ResX));
	OutMetadata.Emplace(TEXT("ResY"), FString::FromInt(GSystemResolution.ResY));
}

bool UAutomatedPerfTestControllerBase::TryStartCSVProfiler(FString CSVFileName, const FString& DestinationFolder, int32 Frames)
{
#if CSV_PROFILER
	if(FCsvProfiler* const CsvProfiler = FCsvProfiler::Get())
	{
		if(!CSVFileName.EndsWith(".csv"))
		{
			CSVFileName += TEXT(".csv");
		}

		UE_LOGF(LogAutomatedPerfTest, Log, "Attempting to start CSV Profile to file %ls", *CSVFileName);

		TArray<TPair<FString, FString>> Metadata;
		GatherTestMetadata(Metadata);
		for (const TPair<FString, FString>& MetadataPair : Metadata)
		{
			CsvProfiler->SetMetadata(*MetadataPair.Key, *MetadataPair.Value);
		}

		// Select the destination folder for csv files. 
		FString CSVDestinationFolder;
		if (!DestinationFolder.IsEmpty())
		{
			CSVDestinationFolder = DestinationFolder;
		}
		else if (!ArtifactOutputPath.IsEmpty())
		{
			CSVDestinationFolder = FPaths::Combine(ArtifactOutputPath, "CSV");
		}
		else
		{
			CSVDestinationFolder = "";
		}

		UE_LOGF(LogGauntlet, Log, "CSV Profiler Destination Filename: %ls, Destination Folder: %ls", 
				*CSVFileName, CSVDestinationFolder.IsEmpty()? TEXT("<Default>") : *CSVDestinationFolder);
		
		CsvProfiler->BeginCapture(Frames, CSVDestinationFolder, CSVFileName);
		CsvProfiler->SetDeviceProfileName(UDeviceProfileManager::Get().GetActiveDeviceProfileName());

		// BeginCapture queues a capture to start on a frame boundary. It isn't possible to verify the capture
		// is now running when this function returns.
		return true;
	}
#endif
	UE_LOGFMT(LogAutomatedPerfTest, Warning, "CSVProfiler Start requested, but not available.");
	return false;
}

bool UAutomatedPerfTestControllerBase::TryStopCSVProfiler()
{
#if CSV_PROFILER
	if(FCsvProfiler* const CsvProfiler = FCsvProfiler::Get())
	{
		const bool bSafeToStopCsvProfiler = CsvProfiler->IsCapturing() && !CsvProfiler->IsEndCapturePending();
		if (bSafeToStopCsvProfiler)
		{
			// Store the event representing the most recent request to end a capture. Waiting on the last handle is 
			// sufficient to ensure all files are processed.
			CsvProfileEndCaptureEvent = FGraphEvent::CreateGraphEvent();
			CsvProfiler->EndCapture(CsvProfileEndCaptureEvent);
		}
		return true;
	}
#endif
	UE_LOGFMT(LogAutomatedPerfTest, Warning, "CSVProfiler Stop requested, but not available.");
	return false;
}

bool UAutomatedPerfTestControllerBase::TryStartFPSChart()
{
	// don't open the folder the FPS chart gets sent to on exit, as it can cause issues when running unattended
	GEngine->Exec(GetWorld(), TEXT("t.FPSChart.OpenFolderOnDump 0"));
	GEngine->StartFPSChart(*GetOverallRegionName(), false);

	return true;
}

bool UAutomatedPerfTestControllerBase::TryStopFPSChart()
{
	GEngine->StopFPSChart(*GetOverallRegionName());
	return true;
}

bool UAutomatedPerfTestControllerBase::TryStartVideoCapture()
{
	if (IVideoRecordingSystem* const VideoRecordingSystem = IPlatformFeaturesModule::Get().GetVideoRecordingSystem())
	{
		const EVideoRecordingState RecordingState = VideoRecordingSystem->GetRecordingState();

		if (RecordingState == EVideoRecordingState::None)
		{
			VideoRecordingSystem->EnableRecording(true);
			
			VideoRecordingTitle = FText::FromString(FPaths::Combine(FPaths::ProjectSavedDir(), GetTestID()));
			const FVideoRecordingParameters VideoRecordingParameters(VideoRecordingSystem->GetMaximumRecordingSeconds(), true, false, false, FPlatformMisc::GetPlatformUserForUserIndex(0));
			VideoRecordingSystem->NewRecording(*GetTestID(), VideoRecordingParameters);

			if (VideoRecordingSystem->IsEnabled())
			{
				if (VideoRecordingSystem->GetRecordingState() == EVideoRecordingState::Starting || VideoRecordingSystem->GetRecordingState() == EVideoRecordingState::Recording)
				{
					UE_LOGF(LogAutomatedPerfTest, Log, "Starting video recording %ls...", *GetTestID());
					return true;
				}
				UE_LOGF(LogAutomatedPerfTest, Warning, "Failed to start video recording %ls. Current state is %i", *GetTestID(), EnumToUnderlyingType(VideoRecordingSystem->GetRecordingState()));
			}
			else
			{
				UE_LOGFMT(LogAutomatedPerfTest, Warning, "Video recording could not be enabled.");
			}
		}
		else
		{
			UE_LOGFMT(LogAutomatedPerfTest, Warning, "Could not start a new recording, may be already recording.");
		}
	}
	else
	{
		UE_LOGFMT(LogAutomatedPerfTest, Warning, "Video recording system is null.");
	}

	return false;	
}

bool UAutomatedPerfTestControllerBase::TryFinalizingVideoCapture(const bool bStopAutoContinue/*=false*/)
{
	if (IVideoRecordingSystem* const VideoRecordingSystem = IPlatformFeaturesModule::Get().GetVideoRecordingSystem())
	{
		if (VideoRecordingSystem->GetRecordingState() != EVideoRecordingState::None)
		{
			VideoRecordingSystem->FinalizeRecording(true, VideoRecordingTitle, FText::GetEmpty(), bStopAutoContinue);

			if (VideoRecordingSystem->GetRecordingState() == EVideoRecordingState::Finalizing)
			{
				UE_LOGFMT(LogAutomatedPerfTest, Log, "Finalizing recording...");
				VideoRecordingSystem->GetOnVideoRecordingFinalizedDelegate().AddUObject(this, &ThisClass::OnVideoRecordingFinalized);
				return true;
			}
			else
			{
				UE_LOGF(LogAutomatedPerfTest, Warning, "Attempted to finalize video recording, but current state %i is not %i", EnumToUnderlyingType(VideoRecordingSystem->GetRecordingState()), EnumToUnderlyingType(EVideoRecordingState::Finalizing))
			}
		}
		else
		{
			UE_LOGF(LogAutomatedPerfTest, Warning, "Attempted to finalize video recording, but state is %i", EnumToUnderlyingType(VideoRecordingSystem->GetRecordingState()))
		}
	}
	else
	{
		UE_LOGFMT(LogAutomatedPerfTest, Warning, "Video recording system is null.");
	}

	return false;	
}

void UAutomatedPerfTestControllerBase::SetupTest()
{
	UE_LOGFMT(LogAutomatedPerfTest, Log, "Base:: SetupTest");

	SetupGameModeInstance();

	// Subclasses should implement their own transitions from SetupTest to RunTest depending on their needs
	UE_LOGF(LogGauntlet, Log, "Setup Test: %ls", *GetTestID());
}

void UAutomatedPerfTestControllerBase::RunTest()
{
	UE_LOGFMT(LogAutomatedPerfTest, Log, "Base:: RunTest");

	if(GameMode && GameMode->GetClass()->ImplementsInterface(UAutomatedPerfTestInterface::StaticClass()))
	{
		IAutomatedPerfTestInterface::Execute_RunTest(GameMode);
	}

	UE_LOGF(LogGauntlet, Log, "Running Test: %ls", *GetTestID());
}

void UAutomatedPerfTestControllerBase::TeardownTest(bool bExitAfterTeardown)
{
	UE_LOGFMT(LogAutomatedPerfTest, Log, "Base:: TeardownTest");

	if(GameMode && GameMode->GetClass()->ImplementsInterface(UAutomatedPerfTestInterface::StaticClass()))
	{
		IAutomatedPerfTestInterface::Execute_TeardownTest(GameMode);
	}

	if(bExitAfterTeardown)
	{
		TriggerExitAfterDelay();
	}

	UE_LOGF(LogGauntlet, Log, "Teardown Test: %ls", *GetTestID());
}

void UAutomatedPerfTestControllerBase::TriggerExitAfterDelay()
{
	const float Delay = GetDefault<UAutomatedPerfTestProjectSettings>()->TeardownToExitDelay;

	// Gauntlet test controllers are UObject types.
	TObjectPtr<UAutomatedPerfTestControllerBase> ControllerInstance(this);
	FTSTicker::GetCoreTicker().AddTicker(TEXT("ReplayComplete"), Delay, [ControllerInstance](float)
	{
		// Capturing this here is safe as this controller is only destroyed after 
		// Exit is called below and the Gauntlet test ends. When running Gauntlet 
		// tests, the controller instances are owned by the Gauntlet Module 
		if(ControllerInstance)
		{
			ControllerInstance->Exit();
		}
		return false;
	});
}

void UAutomatedPerfTestControllerBase::Exit()
{
	if(GameMode && GameMode->GetClass()->ImplementsInterface(UAutomatedPerfTestInterface::StaticClass()))
	{
		IAutomatedPerfTestInterface::Execute_Exit(GameMode);
	}

	// Wait to end the performance test and request shutdown if out artifacts are still being processed.
	FGraphEventArray Prerequisites;

	// Wait for CSVProfiler to finish processing it's last file.
	if (CsvProfileEndCaptureEvent.IsValid())
	{
		Prerequisites.Add(CsvProfileEndCaptureEvent);
	}

	TWeakObjectPtr<UAutomatedPerfTestControllerBase> WeakThis(this);
	FFunctionGraphTask::CreateAndDispatchWhenReady(
	    [WeakThis]()
	    {
		    UAutomatedPerfTestControllerBase* This = WeakThis.Get();
		    if (!This)
		    {
			    return;
		    }

			// Exit the test, requesting the process exit.
		    This->EndTestSuccess();
	    },
	    TStatId(),
	    &Prerequisites,
	    ENamedThreads::GameThread
	);
}

AGameModeBase* UAutomatedPerfTestControllerBase::GetGameMode() const
{
	return GameMode;
}

void UAutomatedPerfTestControllerBase::TakeScreenshot(FString ScreenshotName)
{
	if(RequestsInsightsTrace())
	{
		// trace screenshots are disabled in shipping by default
#if UE_SCREENSHOT_TRACE_ENABLED
		FTraceScreenshot::RequestScreenshot(ScreenshotName, false, LogAutomatedPerfTest);
#endif
	}
	else
	{
		FScreenshotRequest::RequestScreenshot(ScreenshotName, false, false);
	}
}

void UAutomatedPerfTestControllerBase::SetupGameModeInstance()
{
	GameMode = GetWorld() ? GetWorld()->GetAuthGameMode() : NULL;

	if (GameMode && GameMode->GetClass()->ImplementsInterface(UAutomatedPerfTestInterface::StaticClass()))
	{
		IAutomatedPerfTestInterface::Execute_SetupTest(GameMode);
	}
}

APlayerController* UAutomatedPerfTestControllerBase::GetPlayerController()
{
	UWorld* World = GetWorld();
	if (!World)
	{
		return nullptr;
	}

	if (ULocalPlayer* Player = GEngine->GetLocalPlayerFromControllerId(World, 0))
	{
		FLocalPlayerContext Context(Player);
		if (APlayerController* Controller = Context.GetPlayerController())
		{
			return Controller;
		}
	}

	return nullptr;
}

void UAutomatedPerfTestControllerBase::SetupProfiling()
{
	if (RequestsFPSChart())
	{
		TryStartFPSChart();
	}

	if (RequestsVideoCapture())
	{
		TryStartVideoCapture();
	}
}

void UAutomatedPerfTestControllerBase::InitializeInsights()
{
	if (RequestsInsightsTrace())
	{
		TryStartInsightsTrace();
	}
}

void UAutomatedPerfTestControllerBase::ShutdownInsights()
{
	if (RequestsInsightsTrace())
	{
		TryStopInsightsTrace();
	}
}

void UAutomatedPerfTestControllerBase::MarkProfilingStart()
{
	if (RequestsInsightsTrace())
	{
		InsightsRegionID = TRACE_BEGIN_REGION_WITH_ID(*GetOverallRegionName(), TEXT("AutomatedPerfTest"));
	}
}

void UAutomatedPerfTestControllerBase::TeardownProfiling()
{
	if (RequestsFPSChart())
	{
		TryStopFPSChart();
	}

	if (RequestsVideoCapture())
	{
		if (!TryFinalizingVideoCapture())
		{
			UE_LOGFMT(LogAutomatedPerfTest, Warning, "Attempted to finalize requested video capture, but failed.");
		}
	}
}

void UAutomatedPerfTestControllerBase::MarkProfilingEnd()
{
	if (RequestsInsightsTrace())
	{
		if (InsightsRegionID != 0)
		{
			TRACE_END_REGION_WITH_ID(InsightsRegionID);
		}
	}
}

void UAutomatedPerfTestControllerBase::OnInit()
{
	Super::OnInit();

	UE_LOGFMT(LogAutomatedPerfTest, Log, "Base:: OnInit");

	// don't stop on separator because this will come in comma-separated
	FParse::Value(FCommandLine::Get(), TEXT("AutomatedPerfTest.TraceChannels="), TraceChannels, false);
	
	FParse::Value(FCommandLine::Get(), TEXT("AutomatedPerfTest.TestID="), TestID);
	
	if (FParse::Param(FCommandLine::Get(), TEXT("AutomatedPerfTest.DoInsightsTrace")))
	{
		UE_LOGFMT(LogAutomatedPerfTest, Log, "Insights Trace Requested");
		bRequestsInsightsTrace = true;
	}
	if (FParse::Param(FCommandLine::Get(), TEXT("AutomatedPerfTest.DoCSVProfiler")))
	{
		UE_LOGFMT(LogAutomatedPerfTest, Log, "CSV Profiler Requested");
		bRequestsCSVProfiler = true;
	}
	if (FParse::Param(FCommandLine::Get(), TEXT("AutomatedPerfTest.DoFPSChart")))
	{
		UE_LOGFMT(LogAutomatedPerfTest, Log, "FPSCharts Requested");
		bRequestsFPSChart = true;
	}
	if (FParse::Param(FCommandLine::Get(), TEXT("AutomatedPerfTest.DoVideoCapture")))
	{
		UE_LOGFMT(LogAutomatedPerfTest, Log, "Video Capture Requested");
		bRequestsVideoCapture = true;
	}
	if (FParse::Param(FCommandLine::Get(), TEXT("AutomatedPerfTest.LockDynamicRes")))
	{
		UE_LOGFMT(LogAutomatedPerfTest, Log, "Locking dynamic res requested");
		bRequestsLockedDynRes = true;
	}
	if (FParse::Value(FCommandLine::Get(), TEXT("AutomatedPerfTest.ArtifactOutputPath="), ArtifactOutputPath))
	{
		UE_LOGF(LogAutomatedPerfTest, Log, "Artifact Output Path: %ls", *ArtifactOutputPath)
	}

#if CSV_PROFILER
	// Translate the AutomatedPerfTest internal list of CsvMetadata provided by the Gauntlet node to CsvProfiler calls.
	// This is provided here as well to separate systemic use by APT controllers from user use of CsvProfiler provides 
	// csvMetadata= support.
	FString CsvMetadataStr;
	if (FParse::Value(FCommandLine::Get(), TEXT("AutomatedPerfTest.CsvMetadata="), CsvMetadataStr, false))
	{
		TArray<FString> CsvMetadataList;
		CsvMetadataStr.ParseIntoArray(CsvMetadataList, TEXT(","), true);
		for (int i = 0; i < CsvMetadataList.Num(); i++)
		{
			const FString& Metadata = CsvMetadataList[i];
			FString Key;
			FString Value;
			if (Metadata.Split(TEXT("="), &Key, &Value))
			{
				FCsvProfiler::SetMetadata(*Key, *Value);
			}
		}
	}
#endif

	FWorldDelegates::OnPreWorldInitialization.AddUObject(this, &ThisClass::OnPreWorldInitializeInternal);

	InitializeInsights();

	UE_LOGF(LogGauntlet, Log, "Initializing Test: %ls", *GetTestID());
}

void UAutomatedPerfTestControllerBase::OnTick(float TimeDelta)
{
	Super::OnTick(TimeDelta);
	
	MarkHeartbeatActive();
}

void UAutomatedPerfTestControllerBase::BeginDestroy()
{
	UnbindAllDelegates();
	
	Super::BeginDestroy();
}

void UAutomatedPerfTestControllerBase::EndTestSuccess() 
{ 
	EndAutomatedPerfTest(0); 
}

void UAutomatedPerfTestControllerBase::EndTestFailure(const int32 ExitCode) 
{ 
	EndAutomatedPerfTest(ExitCode); 
}

void UAutomatedPerfTestControllerBase::EndAutomatedPerfTest(const int32 ExitCode)
{
	UnbindAllDelegates();
	ShutdownInsights();

	UE_LOGF(LogAutomatedPerfTest, Log, "Test ID %ls completed, requesting exit...", *GetTestID());
	
	EndTest(ExitCode);
}

void UAutomatedPerfTestControllerBase::OnVideoRecordingFinalized(bool Succeeded, const FString& FilePath)
{
	if(!Succeeded)
	{
		UE_LOGFMT(LogAutomatedPerfTest, Warning, "Video Capture finalized, but did not succeed");
	}
/* TODO moving this to automation layer due to file access restrictions
	FString SrcFilePath;
	FString FileName;
	FString Extension;
	FPaths::Split(FilePath, SrcFilePath, FileName, Extension);

	FString DestFileName = FileName + "." + Extension;
	
	const FString DestinationDir = FPaths::Combine(FPaths::ProjectSavedDir(), "Videos");
	const FString DestinationFilePath = FPaths::Combine(DestinationDir, DestFileName);
	
	UE_LOGF(LogAutomatedPerfTest, Log, "Copying video file %ls to Saved: %ls", *FilePath, *DestinationFilePath);
	
	if(IFileManager::Get().Copy(*DestinationFilePath, *FilePath, 1, 1) != COPY_OK)
	{
		UE_LOGFMT(LogAutomatedPerfTest, Warning, "Failed to copy video file");
	}
	*/
}

void UAutomatedPerfTestControllerBase::UnbindAllDelegates()
{
	if(UWorld* const World = GetWorld())
	{
		World->OnWorldBeginPlay.RemoveAll(this);
		World->GameStateSetEvent.RemoveAll(this);
	}

#if CSV_PROFILER
	if (FCsvProfiler* const CsvProfiler = FCsvProfiler::Get())
	{
		CsvProfiler->OnCSVProfileFinished().Remove(CsvProfilerDelegateHandle);
	}
#endif // CSV_PROFILER

	if(RequestsVideoCapture())
	{
		if (IVideoRecordingSystem* const VideoRecordingSystem = IPlatformFeaturesModule::Get().GetVideoRecordingSystem())
		{
			VideoRecordingSystem->GetOnVideoRecordingFinalizedDelegate().RemoveAll(this);
		}
	}
}

void UAutomatedPerfTestControllerBase::ConsoleCommand(const TCHAR* Cmd)
{
	if (APlayerController* Controller = GetPlayerController())
	{
		Controller->ConsoleCommand(Cmd);
	}

	UE_LOGF(LogAutomatedPerfTest, Display, "Issued ConsoleCommand '%ls'", Cmd);
}
