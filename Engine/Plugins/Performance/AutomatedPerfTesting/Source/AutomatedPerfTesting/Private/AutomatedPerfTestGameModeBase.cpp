// Copyright Epic Games, Inc. All Rights Reserved.

#include "AutomatedPerfTestGameModeBase.h"
#include "AutomatedPerfTesting.h"
#include "Logging/StructuredLog.h"

void AAutomatedPerfTestGameModeBase::SetupTest_Implementation()
{
	UE_LOGFMT(LogAutomatedPerfTest, Log, "AutomatedPerfTestGameModeBase SetupTest");
}

void AAutomatedPerfTestGameModeBase::TeardownTest_Implementation()
{
	UE_LOGFMT(LogAutomatedPerfTest, Log, "AutomatedPerfTestGameModeBase TeardownTest");
}

void AAutomatedPerfTestGameModeBase::RunTest_Implementation()
{
	UE_LOGFMT(LogAutomatedPerfTest, Log, "AutomatedPerfTestGameModeBase RunTest");
}

void AAutomatedPerfTestGameModeBase::Exit_Implementation()
{
	UE_LOGFMT(LogAutomatedPerfTest, Log, "AutomatedPerfTestGameModeBase Exit");
}