// Copyright Epic Games, Inc. All Rights Reserved.

#include "AutomatedPerfTestInterface.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AutomatedPerfTestInterface)

// Add default functionality here for any IAutomatedPerfTestInterface functions that are not pure virtual.
