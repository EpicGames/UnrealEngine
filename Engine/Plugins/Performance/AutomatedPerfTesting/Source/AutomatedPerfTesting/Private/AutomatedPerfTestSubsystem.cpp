// Copyright Epic Games, Inc. All Rights Reserved.

#include "AutomatedPerfTestSubsystem.h"

#include "GauntletModule.h"
#include "Modules/ModuleManager.h"
#include "AutomatedPerfTestControllerBase.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AutomatedPerfTestSubsystem)

FString UAutomatedPerfTestSubsystem::GetTestID()
{
	if(FGauntletModule* ParentModule = &FModuleManager::Get().GetModuleChecked<FGauntletModule>(TEXT("Gauntlet")))
	{
		if(UAutomatedPerfTestControllerBase* Controller = ParentModule->GetTestController<UAutomatedPerfTestControllerBase>())
		{
			return Controller->GetTestID();
		}
	}
	return TEXT("");
}
