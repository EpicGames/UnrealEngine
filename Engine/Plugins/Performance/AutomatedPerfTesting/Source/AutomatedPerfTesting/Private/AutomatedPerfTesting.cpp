// Copyright Epic Games, Inc. All Rights Reserved.

#include "AutomatedPerfTesting.h"

#define LOCTEXT_NAMESPACE "FAutomatedPerfTestingModule"

void FAutomatedPerfTestingModule::StartupModule()
{
}

void FAutomatedPerfTestingModule::ShutdownModule()
{
}

#undef LOCTEXT_NAMESPACE
IMPLEMENT_MODULE(FAutomatedPerfTestingModule, AutomatedPerfTesting)