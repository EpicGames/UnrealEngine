// Copyright Epic Games, Inc. All Rights Reserved.

#include "AutomatedProfileGoTest.h"

#include "CoreMinimal.h"
#include "ProfilingDebugging/CsvProfiler.h"
#include "ProfilingDebugging/MallocLeakReporter.h"
#include "ProfilingDebugging/TraceAuxiliary.h"

#include "GameFramework/PlayerController.h"
#include "Misc/CommandLine.h"
#include "Misc/Paths.h"
#include "JsonObjectConverter.h"
#include "Logging/StructuredLog.h"
#include "UObject/UObjectIterator.h"
#include "TimerManager.h"

#include "AutomatedPerfTesting.h"
#include "AutomatedPerfTestProjectSettings.h"

#include "ProfileGo/ProfileGo.h"
#include "ProfileGo/ProfileGoSubsystem.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AutomatedProfileGoTest)

static FAutoConsoleCommandWithWorldAndArgs ProfileGoConsoleCommand(
	TEXT("apt.profilego"),
	TEXT("Moved to the specified profiling spot"),
	FConsoleCommandWithWorldAndArgsDelegate::CreateStatic(
		[](const TArray<FString>& Params, UWorld* World)
		{
			if (Params.Num() == 0)
			{
				return;
			}

			FString ExtraArgs;
			for (int32 i = 1; i < Params.Num(); i++)
			{
				ExtraArgs += TEXT(" ") + Params[i];
			}

			if (UProfileGoSubsystem* ProfileGoSubsystem = World->GetSubsystem<UProfileGoSubsystem>())
			{
				ProfileGoSubsystem->Run<UProfileGo>(*Params[0], *ExtraArgs);
			}
		}
	)
);

UAutomatedProfileGoTest::UAutomatedProfileGoTest(const FObjectInitializer& ObjectInitializer)
	: UAutomatedPerfTestControllerBase(ObjectInitializer)
{
	bPlayerSetupFinished = false;
	bIgnorePawn = false;
	bBoundDelegates = false;
	bRequestedProfileGo = false;
	bStartedProfileGo = false;
	bShouldTick = true;
	NumRequiredLoops = 0;
	NumCompletedLoops = 0;
}

void UAutomatedProfileGoTest::OnInit()
{
	const TCHAR* CommandLine = FCommandLine::Get();
	FParse::Value(CommandLine, TEXT("profilegocmd="), ProfileGoCommand);
	ProfileGoCommand = ProfileGoCommand.IsEmpty() ? TEXT("apt.profilego") : ProfileGoCommand;
	// FParse::Value excludes outermost quotes. Ex: -profilego= "foo, bar" ==> foo, bar
	FParse::Value(CommandLine, TEXT("profilego="), ProfileGoScenario, false);

	UE_LOGFMT(LogAutomatedPerfTest, Log, "Profilego scenario: {ScenarioName}", *ProfileGoScenario);
	bIgnorePawn = FParse::Param(CommandLine, TEXT("profilego.ignorepawn"));

	FParse::Value(CommandLine, TEXT("profilego.config="), ProfileGoConfigFile);
	if (!FPaths::FileExists(ProfileGoConfigFile))
	{
		ProfileGoConfigFile.Empty();
	}

	FParse::Value(CommandLine, TEXT("profilego.loops="), NumRequiredLoops);
	// convert -exit arg to a fixed loopcount of 1 if no count was specified
	bool ExitArg = FParse::Param(FCommandLine::Get(), TEXT("profilego.exit"));
	if (NumRequiredLoops == 0 && ExitArg)
	{
		NumRequiredLoops = 1;
	}

	Super::OnInit();
}

FString UAutomatedProfileGoTest::GetPerfTestTypeID() const
{
	return TEXT("ProfileGo");
}

void UAutomatedProfileGoTest::SetupTest()
{
	if (UWorld* World = GetWorld())
	{
		Super::SetupTest();

		UProfileGoSubsystem* ProfileGoSubsystem = World->GetSubsystem<UProfileGoSubsystem>();
		if (!ProfileGoSubsystem)
		{
			return;
		}

		ProfileGoSubsystem->OnRequestFailed().AddUObject(this, &UAutomatedProfileGoTest::OnRequestFailed);
		ProfileGoSubsystem->OnScenarioStarted().AddUObject(this, &UAutomatedProfileGoTest::OnScenarioStarted);
		ProfileGoSubsystem->OnScenarioEnded().AddUObject(this, &UAutomatedProfileGoTest::OnScenarioEnded);
		ProfileGoSubsystem->OnPassStarted().AddUObject(this, &UAutomatedProfileGoTest::OnPassStarted);
		ProfileGoSubsystem->OnPassEnded().AddUObject(this, &UAutomatedProfileGoTest::OnPassEnded);
		ProfileGoSubsystem->SetTestController(this);

		if (!ProfileGoConfigFile.IsEmpty())
		{
			ProfileGoSubsystem->LoadFromJSON(ProfileGoConfigFile);
			ProfileGoConfigFile.Empty();
		}

		// Reset if loading into new world
		bRequestedProfileGo = false; 
	}

}

void UAutomatedProfileGoTest::OnTick(float TimeDelta)
{
	if (!bShouldTick)
	{
		return;
	}

	if (!GetWorld())
	{
		return;
	}

	UProfileGoSubsystem* ProfileGoSubsystem = GetWorld()->GetSubsystem<UProfileGoSubsystem>();
	if (!ProfileGoSubsystem)
	{
		return;
	}

	const APlayerController* PlayerController = Cast<APlayerController>(GetPlayerController());

	// If we are ready to start, make the ProfileGo request
	if (!bRequestedProfileGo)
	{
		if (UProfileGoSubsystem::WaitForLoadingAndStreaming(PlayerController->GetWorld(), TimeDelta))
		{
			return;
		}

		if (!bPlayerSetupFinished)
		{
			// TODO: Optional Delegate call for Player Setup
			bPlayerSetupFinished = true;
		}

		UE_LOGFMT(LogAutomatedPerfTest, Log, "Starting ProfileGo {ScenarioName} run {TestIteration} of {TotalTestCount}", *ProfileGoScenario, NumCompletedLoops + 1, NumRequiredLoops);

		FString Args = GetProfileGoArgsString();
		FString FullProfileGoCommandString = ProfileGoCommand + " " + ProfileGoScenario + " " + Args;
		ConsoleCommand(*FullProfileGoCommandString);
		// If the console command fails, UAutomatedProfileGoTest::OnRequestFailed() will be called and end the test
		bRequestedProfileGo = true;
		return;
	}

	// ProfileGo will call OnPassStarted(). Then we monitor and wait for exit.
	if (!bStartedProfileGo)
	{
		return;
	}

	// monitor state change
	FString Status = ProfileGoSubsystem->GetStatusMessage();
	if (Status != ProfileGoStatus)
	{
		ProfileGoStatus = Status;
		UE_LOGFMT(LogAutomatedPerfTest, Log, "ProfileGo: {ProfileGoStatus}", *ProfileGoStatus);
	}

	// Once finished, do final cleanup
	if (ProfileGoSubsystem->IsRunning())
	{
		return;
	}

	if (ProfileGoSubsystem->HasEncounteredError())
	{
		UE_LOGFMT(LogAutomatedPerfTest, Error, "ProfileGo failed with error : {ProfileGoStatus}", *Status);
		CompleteTest(false);
	}
	else
	{
		NumCompletedLoops++;

		UE_LOGFMT(LogAutomatedPerfTest, Log, "Completed loop {NumCompletedLoops} of {NumRequiredLoops}.", NumCompletedLoops, NumRequiredLoops);

		if (NumRequiredLoops > 0)
		{
			// TODO -profilego.loops= and -profilego.exit params are not handled within UProfileGo. Should we move them there?
			if (NumCompletedLoops < NumRequiredLoops)
			{
				bRequestedProfileGo = false;
				bStartedProfileGo = false;
			}
			else
			{
				UE_LOGFMT(LogAutomatedPerfTest, Log, "Completed ProfileGo. Exiting");
				CompleteTest(true);
			}
		}
	}

	Super::OnTick(TimeDelta);
}

void UAutomatedProfileGoTest::OnRequestFailed()
{
	UE_LOGFMT(LogAutomatedPerfTest, Error, "ProfileGo request failed. Exiting.");
	CompleteTest(false);
}

void UAutomatedProfileGoTest::OnScenarioStarted(const FString& InName)
{
	MarkHeartbeatActive(FString::Printf(TEXT("Starting %s"), *InName));
}

void UAutomatedProfileGoTest::OnScenarioEnded(const FString& InName)
{
}

void UAutomatedProfileGoTest::OnPassStarted()
{
	bStartedProfileGo = true;
}

void UAutomatedProfileGoTest::OnPassEnded()
{
}

void UAutomatedProfileGoTest::UnbindAllDelegates()
{
	if (UWorld* World = GetWorld())
	{
		if (UProfileGoSubsystem* ProfileGoSubsystem = World->GetSubsystem<UProfileGoSubsystem>())
		{
			ProfileGoSubsystem->OnRequestFailed().RemoveAll(this);
			ProfileGoSubsystem->OnScenarioStarted().RemoveAll(this);
			ProfileGoSubsystem->OnScenarioEnded().RemoveAll(this);
			ProfileGoSubsystem->OnPassStarted().RemoveAll(this);
			ProfileGoSubsystem->OnPassEnded().RemoveAll(this);
			ProfileGoSubsystem->SetTestController(nullptr);
		}
	}

	Super::UnbindAllDelegates();
}

void UAutomatedProfileGoTest::CompleteTest(bool bSuccess)
{
	constexpr float RateInSeconds = 5.0f;

	// Handle this potentially being destroyed priorty to the timer expiring.
	FTimerDelegate Delegate = FTimerDelegate::CreateWeakLambda(this, [this, bSuccess]()
	{
		if (bSuccess)
		{
			EndTestSuccess();
		}
		else
		{
			EndTestFailure();
		}
	});

	FTimerHandle WaitTimerHandle;
	GWorld->GetTimerManager().SetTimer(WaitTimerHandle, Delegate, RateInSeconds, false);
	bShouldTick = false;
}

void UAutomatedProfileGoTest::LoadFromJSON(const FString& Filename)
{
	if (UWorld* World = GetWorld())
	{
		if (UProfileGoSubsystem* ProfileGoSubsystem = World->GetSubsystem<UProfileGoSubsystem>())
		{
			ProfileGoSubsystem->LoadFromJSON(Filename);
		}
	}
}

void UAutomatedProfileGoTest::SaveToJSON(const FString& Filename)
{
	if (UWorld* World = GetWorld())
	{
		if (UProfileGoSubsystem* ProfileGoSubsystem = World->GetSubsystem<UProfileGoSubsystem>())
		{
			ProfileGoSubsystem->SaveToJSON(Filename);
		}
	}
}

FString UAutomatedProfileGoTest::GetProfileGoArgsString()
{
	FString ProfileGoArgs;
	const TCHAR* CommandLine = FCommandLine::Get();

	// Parse all command line params that begin with "profilego." and convert them into an ArgsString for ProfileGo()
	FString Token;
	while (FParse::Token(CommandLine, Token, true))
	{
		Token.RemoveFromStart("-");

		// ExtraArgs is a special case to pass through a raw chunk of command line
		if (Token.RemoveFromStart("profilego.extraargs=", ESearchCase::IgnoreCase))
		{
			UE_LOGFMT(LogAutomatedPerfTest, Log, "Found profilego token: -extraargs={ExtraArgs}", *Token);
			FString UnquotedValue;
			if (FParse::QuotedString(*Token, UnquotedValue))
			{
				Token = UnquotedValue;
				UE_LOGFMT(LogAutomatedPerfTest, Log, "Parsed extraargs token into: {ParsedExtraArgs}", *UnquotedValue);
			}
			ProfileGoArgs += " " + Token;
		}
		else if (Token.RemoveFromStart("profilego.", ESearchCase::IgnoreCase))
		{
			UE_LOGFMT(LogAutomatedPerfTest, Log, "Found ProfileGo token: -{ProfileGoToken}", *Token);
			ProfileGoArgs += " -" + Token;
		}
	}

	return ProfileGoArgs.TrimStart();
}

UAutomatedProfileGoTestProjectSettings::UAutomatedProfileGoTestProjectSettings(const FObjectInitializer& Initializer)
	: Super(Initializer)
{

}
