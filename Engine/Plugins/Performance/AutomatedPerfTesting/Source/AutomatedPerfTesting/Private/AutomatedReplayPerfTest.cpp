// Copyright Epic Games, Inc. All Rights Reserved.

#include "AutomatedReplayPerfTest.h"

#include "Engine/World.h"
#include "Engine/Engine.h"
#include "Engine/GameInstance.h"
#include "Logging/StructuredLog.h"
#include "Misc/CommandLine.h"
#include "Misc/Paths.h"
#include "Net/UnrealNetwork.h"
#include "GameFramework/GameModeBase.h"

#include "AutomatedPerfTesting.h"
#include "AutomatedPerfTestProjectSettings.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AutomatedReplayPerfTest)

UAutomatedReplayPerfTestProjectSettings::UAutomatedReplayPerfTestProjectSettings(const FObjectInitializer& Initializer)
	: Super(Initializer)
{
}

bool UAutomatedReplayPerfTestProjectSettings::GetReplayPathFromName(FName TestName, FString& FoundReplay) const
{
	const FString TestReplay = TestName.ToString();
	for (const FFilePath& Replay : ReplaysToTest)
	{
		const FString& ReplayPath = Replay.FilePath;
		if (ReplayPath.IsEmpty())
		{
			continue;
		}

		if (ReplayPath.Contains(TestReplay))
		{
			// Note: Unfortunately, some platforms may need this path to be
			// updated depending on host mounting requirements of the platform. 
			// Some platforms may need this file copied to the device itself. 
			// Gauntlet controller handles this logic for you. But if for some 
			// reason this test is run without gauntlet, you may need to update 
			// this path and/or copy over the replay files manually. 
			FoundReplay = FPaths::Combine(FPaths::ProjectDir(), ReplayPath);
			return FPaths::FileExists(FoundReplay);
		}
	}

	return false;
}

FString UAutomatedReplayPerfTest::GetPerfTestTypeID() const
{
	return TEXT("Replay");
}

void UAutomatedReplayPerfTest::SetupTest()
{
	bIsTearingDown = false;

	if (UWorld* World = GetWorld())
	{
		// If this is true, it usually means replay is already 
		// running and has loaded a new map and SetupTest is
		// being called again on World OnBeginPlay
		if (bIsReplayTriggered)
		{
			// We need to ensure we have the right Game Mode instance
			// after transitioning between worlds.
			SetupGameModeInstance();
			bIsReplayTriggered = false;
			return;
		}

		// Register delegates to be called when replay playback has started or 
		// completed or if there is a failure for some reason.
		FNetworkReplayDelegates::OnReplayStarted.AddUObject(this, &ThisClass::OnReplayStarted);
		FNetworkReplayDelegates::OnReplayPlaybackComplete.AddUObject(this, &ThisClass::OnReplayComplete);
		FNetworkReplayDelegates::OnReplayPlaybackFailure.AddUObject(this, &ThisClass::OnReplayFailure);

		UE_LOGFMT(LogAutomatedPerfTest, Log, "Starting Replay Perf Test");
		RunTest();
	}
	else
	{
		UE_LOGFMT(LogAutomatedPerfTest, Error, "Invalid World when starting AutomatedReplayPerfTest, exiting...");
		EndTestFailure();
	}
}

void UAutomatedReplayPerfTest::RunTest()
{
	UWorld* World = GetWorld();
	UGameInstance* GameInstance = World ? World->GetGameInstance() : nullptr;
	if (GameInstance)
	{
		bIsReplayTriggered = GameInstance->PlayReplay(ReplayName);

		// Also check if replay playback was deferred for any reason. 
		bIsReplayTriggered = bIsReplayTriggered || GameInstance->IsReplayDeferred(ReplayName);
	}

	if (!bIsReplayTriggered)
	{
		UE_LOGFMT(LogAutomatedPerfTest, Error, "Could not start Replay Perf Test");
		EndTestFailure();
	}
}

void UAutomatedReplayPerfTest::TeardownTest(bool bExitAfterTeardown)
{
	if (bIsTearingDown)
	{
		// When a replay completes, it can call OnReplayPlaybackComplete multiple times until
		// exit, causing Teardown to be invoked multiple times. This mitigates that issue.
		return;
	}

	UE_LOGFMT(LogAutomatedPerfTest, Log, "AutomatedReplayPerfTest::TeardownTest");

	bIsTearingDown = true;
	MarkProfilingEnd();

	if (RequestsCSVProfiler())
	{
		TryStopCSVProfiler();
	}

	TeardownProfiling();
	Super::TeardownTest(bExitAfterTeardown);
}

void UAutomatedReplayPerfTest::OnInit()
{
	Super::OnInit();

	UE_LOGFMT(LogAutomatedPerfTest, Log, "AutomatedReplayPerfTest::OnInit");

	// Replay subsystem looks for replay files in this folder by default.
	const FString DefaultReplayPath = FPaths::Combine(*FPaths::ProjectSavedDir(), TEXT("Demos/")); 
	const FString DefaultReplayExtension = TEXT(".replay");

	const UAutomatedReplayPerfTestProjectSettings* Settings = GetDefault<UAutomatedReplayPerfTestProjectSettings>();

	const auto GetDefaultReplayFromSettings = [&]()
	{
		if (Settings->ReplaysToTest.IsEmpty())
		{
			return FString();
		}

		// Replay path from settings will always be relative to the project
		// directory. Default to first replay in list. 
		return FPaths::Combine(FPaths::ProjectDir(), Settings->ReplaysToTest[0].FilePath);
	};

	FString CmdReplayName;
	FParse::Value(FCommandLine::Get(), TEXT("AutomatedPerfTest.ReplayPerfTest.ReplayName="), CmdReplayName);

	// Attempt to read from settings first if replay file name is not provided. 
	// Otherwise command line takes precedence
	ReplayName = CmdReplayName.IsEmpty() ? GetDefaultReplayFromSettings() : CmdReplayName;

	bool bReplayExists = !ReplayName.IsEmpty() && FPaths::FileExists(ReplayName);
	if (!bReplayExists)
	{
		// On some devices, we may have to copy the replay file to the device
		// as it may not support reading from the host directly. In that case
		// we assume only the replay file name is supplied without the whole 
		// path. We check the default replay path in this scenario. 
		const FString ReplayAlternatePath = DefaultReplayPath + ReplayName + DefaultReplayExtension;
		bReplayExists = FPaths::FileExists(ReplayAlternatePath);
		if (bReplayExists)
		{
			UE_LOGF(LogAutomatedPerfTest, Log, "Replay found in default Demos path: %ls", *ReplayAlternatePath);
		}
	}

	// Attempt to see if we can retrieve the replay path from 
	// the settings with the given replay name
	if (!bReplayExists && !ReplayName.IsEmpty())
	{
		FString ReplayTest;
		bReplayExists = Settings->GetReplayPathFromName(*ReplayName, ReplayTest);
		UE_LOGF(LogAutomatedPerfTest, Log, "Matching Replay found in settings: %ls", *ReplayName);
	}

	if (ReplayName.IsEmpty() || !bReplayExists)
	{
		UE_LOGFMT(LogAutomatedPerfTest, Error, "Replay not specified in args nor found in settings");
		EndTestFailure();
		return;
	}

	UE_LOGF(LogAutomatedPerfTest, Log, "Replay Name: %ls", *ReplayName);
}

void UAutomatedReplayPerfTest::UnbindAllDelegates()
{
	FNetworkReplayDelegates::OnReplayStarted.RemoveAll(this);
	FNetworkReplayDelegates::OnReplayPlaybackComplete.RemoveAll(this);
	FNetworkReplayDelegates::OnReplayPlaybackFailure.RemoveAll(this);
	Super::UnbindAllDelegates();
}

void UAutomatedReplayPerfTest::OnReplayStarted(UWorld* World)
{
	// Set up Insights/CSV profiler and mark the start events/regions. 
	// We do this here so that metric collection starts only when replay 
	// playback is ready. This is especially useful since Replays can
	// be deferred.
	SetupProfiling();
	MarkProfilingStart();

	if (RequestsCSVProfiler())
	{
		TryStartCSVProfiler(GetTestID());
	}
}

void UAutomatedReplayPerfTest::OnReplayComplete(UWorld* World)
{
	// We manually ensure we tear down and exit here the moment replay
	// playback is completed, otherwise it requires user input to exit. 
	constexpr bool bDelayedExitAfterTeardown = true;
	TeardownTest(bDelayedExitAfterTeardown);
}

void UAutomatedReplayPerfTest::OnReplayFailure(UWorld* World, const UE::Net::TNetResult<EReplayResult>& Error)
{
	UE_LOGF(LogAutomatedPerfTest, Error, "Replay playback error: %ls", *Error.GetErrorContext());

	// End replay run with failure error code. 
	EndTestFailure();
}
