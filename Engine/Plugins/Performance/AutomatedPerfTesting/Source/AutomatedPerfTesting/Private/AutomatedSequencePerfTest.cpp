// Copyright Epic Games, Inc. All Rights Reserved.


#include "AutomatedSequencePerfTest.h"

#include "TimerManager.h"
#include "AutomatedPerfTesting.h"
#include "LevelSequenceActor.h"
#include "LevelSequencePlayer.h"
#include "Engine/LevelStreaming.h"
#include "MovieSceneSequencePlayer.h"
#include "Kismet/GameplayStatics.h"
#include "Logging/StructuredLog.h"
#include "Misc/CommandLine.h"
#include "ProfilingDebugging/CsvProfiler.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AutomatedSequencePerfTest)

/*****
 * UAutomatedSequencePerfTestProjectSettings
 *****/

UAutomatedSequencePerfTestProjectSettings::UAutomatedSequencePerfTestProjectSettings(const FObjectInitializer& Initializer)
	: Super(Initializer)
{
}

bool UAutomatedSequencePerfTestProjectSettings::GetComboFromTestName(FName TestName, FAutomatedPerfTestMapSequenceCombo& FoundSequence) const
{
	for (FAutomatedPerfTestMapSequenceCombo TestCombo : MapsAndSequencesToTest)
	{
		if(TestCombo.ComboName == TestName)
		{
			FoundSequence = TestCombo;
			return true;
		}
	}
	return false;
}


/*****
 * UAutomatedSequencePerfTest
 *****/
void UAutomatedSequencePerfTest::SetupTest()
{
	if(CurrentMapSequenceCombo.IsSet())
	{
		// don't even try to set up the test if we're not in the correct map
		if(GetCurrentMap() == CurrentMapSequenceCombo->Map.GetAssetName())
		{
			// make sure the world exists, then create a sequence player
			if (UWorld* const World = GetWorld())
			{
				// Begin World Fires for each sub level so we need to make sure that all of the required ones are ready to go
				// TODO: Consider pushing this logic into the Base Class
				if (World->GetNumStreamingLevelsBeingLoaded() != 0)
				{
					return;
				}
			
				Super::SetupTest();
				// reset the camera cut number
				NumCameraCuts = -1;
				
				// load the sequence specified by the user
				UE_LOGF(LogAutomatedPerfTest, Log, "Loading sequence %ls", *CurrentMapSequenceCombo->Sequence.ToString());
				ULevelSequence* TargetSequence = LoadObject<ULevelSequence>(NULL, *CurrentMapSequenceCombo->Sequence.ToString(), NULL, LOAD_None, NULL);
				check(TargetSequence);
		
				UE_LOGFMT(LogAutomatedPerfTest, Log, "World is valid, creating sequence player");
				SequencePlayer = ULevelSequencePlayer::CreateLevelSequencePlayer(World, TargetSequence, FMovieSceneSequencePlaybackSettings(), SequenceActor);

				if (SequencePlayer == nullptr)
				{
					UE_LOGFMT(LogAutomatedPerfTest, Error, "Unable to create sequence player when starting AutomatedSequencePerfTest, exiting...");
					EndTestFailure();
					return;
				}

				// set the sequence up at the beginning
				FMovieSceneSequencePlaybackParams PlaybackParams = FMovieSceneSequencePlaybackParams();
				FMovieSceneSequencePlayToParams PlayToParams = FMovieSceneSequencePlayToParams();

				PlaybackParams.Time = 0.0;
				PlaybackParams.UpdateMethod = EUpdatePositionMethod::Scrub;

				UE_LOGFMT(LogAutomatedPerfTest, Log, "SetupMapTest:: Scrubbing to start");
				SequencePlayer->PlayTo(PlaybackParams, PlayToParams);

				// Setup profiling only when we are sure to run the sequence. 
				SetupProfiling();

				UE_LOGF(LogAutomatedPerfTest, Verbose, "SetupMapTest:: Waiting for %f seconds before playing seqeuence", Settings->SequenceStartDelay);
				FTimerHandle UnusedHandle;
				GetWorld()->GetTimerManager().SetTimer(UnusedHandle, this, &UAutomatedSequencePerfTest::RunTest, 1.0, false, Settings->SequenceStartDelay);
			}
			// if we have an invalid world, we can't run the test, so we should bail out
			else
			{
				UE_LOGFMT(LogAutomatedPerfTest, Error, "Invalid World when starting AutomatedSequencePerfTest, exiting...");
				EndTestFailure();
				return;
			}
		}
		else
		{
			UE_LOGF(LogAutomatedPerfTest, Log, "Current Map Name %ls is not expected %ls, calling NextMap.", *GetCurrentMap(), *CurrentMapSequenceCombo->Map.GetAssetName())
			NextMap();
		}
	}
	else
	{
		UE_LOGFMT(LogAutomatedPerfTest, Log, "Current Map Sequence Combo has not been set, calling NextMap to trigger.");
		NextMap();
	}
}

void UAutomatedSequencePerfTest::NextMap()
{
	UE_LOGFMT(LogAutomatedPerfTest, Log, "UAutomatedSequencePerfTest::NextMap");

	if(MapSequenceCombos.Num() > 0)
	{
		CurrentMapSequenceCombo = MapSequenceCombos.Pop();
		UE_LOGF(LogAutomatedPerfTest, Log, "Setting up test for Map/Sequence combo %ls", *CurrentMapSequenceCombo->ComboName.ToString())

		// no need to prepend this with a ? since OpenLevel handles that part for us
		FString OptionsString;
		if(!CurrentMapSequenceCombo->GameModeOverride.IsEmpty())
		{
			UE_LOGF(LogAutomatedPerfTest, Log, "Game Mode overridden to %ls", *CurrentMapSequenceCombo->GameModeOverride)
			OptionsString += "game=" + CurrentMapSequenceCombo->GameModeOverride;
		}
		
		if (OptionsString.IsEmpty() && AutomatedPerfTest::IsWorldLoaded(CurrentMapSequenceCombo->Map.GetAssetName()))
		{
			UE_LOGF(LogAutomatedPerfTest, Log, "%ls is already loaded and does not have any options string, skipping map load and setting up the test", *CurrentMapSequenceCombo->Map.GetAssetName());
			// TODO: Fix up SetupTest's logic to also correctly deal with MultiWorld
			SetupTest();
		}
		else 
		{
			UE_LOGF(LogAutomatedPerfTest, Log, "Opening map %ls%ls", *CurrentMapSequenceCombo->Map.GetAssetName(), *OptionsString);
			UGameplayStatics::OpenLevel(AutomatedPerfTest::FindCurrentWorld(), *CurrentMapSequenceCombo->Map.GetAssetName(), true, OptionsString);
		}
	}
	else
	{
		UE_LOGFMT(LogAutomatedPerfTest, Log, "UAutomatedSequencePerfTest::NextMap, all maps complete, exiting after delay.");
		TriggerExitAfterDelay();
	}
}

void UAutomatedSequencePerfTest::RunTest()
{
	Super::RunTest();

	UE_LOGFMT(LogAutomatedPerfTest, Log, "RunTest");
	
	// make sure we have a valid sequence player
	if(SequencePlayer)
	{
		UE_LOGFMT(LogAutomatedPerfTest, Log, "RunTest::Valid Sequence Player, proceeding");
		MarkProfilingStart(); // Mark profiler start region on sequence start. 

		if (RequestsCSVProfiler())
		{
			FString CSVFilename;
			if (CurrentMapSequenceCombo.IsSet() && !CurrentMapSequenceCombo->ComboName.ToString().IsEmpty())
			{
				CSVFilename = GetTestID() + "_" + CurrentMapSequenceCombo->ComboName.ToString();
			}
			else
			{
				CSVFilename = GetTestID();
				UE_LOGFMT(LogAutomatedPerfTest, Error, "Current Map Sequence Combo Name not set");
			}

			TryStartCSVProfiler(CSVFilename);
		}

		// trigger a camera cut manually in order to start the region for the first camera cut
        SequencePlayer->Play();
		OnCameraCut(SequencePlayer->GetActiveCameraComponent());
		
		// When the sequence has finished, we'll tear down the test in this map via the OnSequenceFinished dispatch
		// because TeardownTest's signature doesn't match OnFinished
        SequencePlayer->OnFinished.AddDynamic(this, &UAutomatedSequencePerfTest::OnSequenceFinished);
		
		SequencePlayer->OnCameraCut.AddDynamic(this, &UAutomatedSequencePerfTest::OnCameraCut);
	}
	// otherwise bail out of the test
	else
	{
		UE_LOGFMT(LogAutomatedPerfTest, Error, "Invalid SequencePlayer when starting AutomatedSequencePerfTest, exiting...");
		EndTestFailure();
	}
}

void UAutomatedSequencePerfTest::OnSequenceFinished()
{
	// trigger OnCameraCut again with a nullptr for the new camera so that
	// we can end the final camera cut's region
	OnCameraCut(nullptr);
	TeardownTest(false);
}

void UAutomatedSequencePerfTest::TeardownTest(bool bExitAfterTeardown)
{
	UE_LOGFMT(LogAutomatedPerfTest, Log, "AutomatedSequencePerfTest::TeardownTest");
	
	MarkProfilingEnd();

	if (RequestsCSVProfiler())
	{
		TryStopCSVProfiler();
	}

	TeardownProfiling();
	Super::TeardownTest(bExitAfterTeardown);

	UnbindAllDelegates();

	// null out the references we have to our world objects
	CurrentCamera = nullptr;
	SequencePlayer = nullptr;
	SequenceActor = nullptr;
	
	NextMap();
}

void UAutomatedSequencePerfTest::GatherTestMetadata(TArray<TPair<FString, FString>>& OutMetadata) const
{
	Super::GatherTestMetadata(OutMetadata);

	// Called after a test is created. CurrentMapSequenceCombo and ComboName are required invariants of this class's 
	// implementation. Missing values result in malformed output.
	checkf(CurrentMapSequenceCombo.IsSet(), TEXT("GatherTestMetadata requires a CurrentMapSequenceCombo."));
	FString ComboName = CurrentMapSequenceCombo->ComboName.ToString();
	checkf(!ComboName.IsEmpty(), TEXT("GatherTestMetadata requires a valid ComboName."));

	OutMetadata.Emplace(TEXT("MapSequenceComboName"), ComboName);
}

void UAutomatedSequencePerfTest::OnCameraCut(UCameraComponent* CameraComponent)
{
	// null check the CurrentCamera so that we can use OnCameraCut to mark the starting camera cut region
	if(CurrentCamera)
	{
		if(RequestsInsightsTrace())
		{
			TRACE_END_REGION(*GetCameraCutID());
		}
#if CSV_PROFILER
		if(RequestsCSVProfiler())
		{
			CSV_EVENT(AutomatedPerfTest, TEXT("END_%s"), *GetCameraCutID())
		}
#endif
	}

	// then null check the new CameraComponent so that we can use OnCameraCut to mark the end of the final camera cut region
	if(CameraComponent)
	{
		NumCameraCuts += 1;
		
		// Then bring in the new camera component for this cut and mark the start of it
		CurrentCamera = CameraComponent;
		if(RequestsInsightsTrace())
		{
			TRACE_BEGIN_REGION(*GetCameraCutID());
		}
#if CSV_PROFILER
		if(RequestsCSVProfiler())
		{
			CSV_EVENT(AutomatedPerfTest, TEXT("START_%s"), *GetCameraCutID())
		}
#endif
	}
}

FString UAutomatedSequencePerfTest::GetPerfTestTypeID() const
{
	return TEXT("Sequence");
}

FString UAutomatedSequencePerfTest::GetCameraCutID()
{
	if(CurrentCamera != nullptr)
	{
		// getting the label of a spawnable camera from Sequencer in a packaged build isn't possible
		// via this method. TODO: find a more reliable way to return camera cut names that are set in Sequencer
		//const AActor* Owner = CurrentCamera->GetOwner();
		//FString CutName = Owner ? Owner->GetActorNameOrLabel() : CurrentCamera->GetName();
		//return CutName;
		FString NumCuts = FString::FromInt(NumCameraCuts);

		// Pad it out to at least four characters for now
		while(NumCuts.Len() < 4)
		{
			NumCuts = "0" + NumCuts;
		}
		
		return FString::Format(TEXT("CameraCut{0}"), {NumCuts});
	}
	return "";
}

void UAutomatedSequencePerfTest::OnInit()
{
	Super::OnInit();
	UE_LOGFMT(LogAutomatedPerfTest, Log, "AutomatedSequencePerfTest::OnInit");
	
	Settings = GetDefault<UAutomatedSequencePerfTestProjectSettings>();

	// if an explicit map/sequence name was set from commandline, use this to override the test
	if (FParse::Value(FCommandLine::Get(), TEXT("AutomatedPerfTest.SequencePerfTest.MapSequenceName="), SequenceTestName))
	{
		FAutomatedPerfTestMapSequenceCombo MapSequenceCombo;
		if(Settings->GetComboFromTestName(SequenceTestName, MapSequenceCombo))
		{
			MapSequenceCombos.Add(MapSequenceCombo);
		}
		else
		{
			UE_LOGF(LogAutomatedPerfTest, Error, "Failed to find MapSequence combo name matching %ls", *SequenceTestName.ToString())
		}
	}
	// otherwise, use all the maps defined in project settings
	else
	{
		for(const FAutomatedPerfTestMapSequenceCombo& MapSequenceCombo : Settings->MapsAndSequencesToTest)
		{
			MapSequenceCombos.Add(MapSequenceCombo);
		}
	}
	
	UE_LOGF(LogAutomatedPerfTest, Log, "Initialized with %d MapSequence combos", MapSequenceCombos.Num());
}

void UAutomatedSequencePerfTest::UnbindAllDelegates()
{
	Super::UnbindAllDelegates();

	UWorld* const World = GetWorld();

	// if we have a valid sequence player, make sure we unbind our events from it when we're wrapping up the test.
	if (SequencePlayer != nullptr)
	{
		SequencePlayer->OnCameraCut.RemoveAll(this);
		SequencePlayer->OnFinished.RemoveAll(this);
		if (World)
		{
			World->GetTimerManager().ClearAllTimersForObject(SequencePlayer);
		}
	}

	// clear any stray timers that might be lying around
	if (World)
	{
		World->GetTimerManager().ClearAllTimersForObject(this);
	}
}
