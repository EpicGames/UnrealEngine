// Copyright Epic Games, Inc. All Rights Reserved.

#include "ProfileGo/ProfileGo.h"
#include "ProfileGo/ProfileGoSubsystem.h"

#include "AutomatedProfileGoTest.h"
#include "AutomatedPerfTesting.h"
#include "AutomatedPerfTestControllerBase.h"
#include "Algo/Find.h"

#include "CoreMinimal.h"
#include "ContentStreaming.h"

#include "Engine/BlockingVolume.h"
#include "Engine/Engine.h"
#include "Engine/LocalPlayer.h"

#include "GameFramework/CharacterMovementComponent.h"
#include "GameFramework/GameState.h"
#include "GameFramework/KillZVolume.h"

#include "HAL/PlatformFileManager.h"
#include "JsonObjectConverter.h"
#include "Kismet/GameplayStatics.h"
#include "UObject/UObjectIterator.h"

#include "Logging/StructuredLog.h"
#include "Logging/LogMacros.h"

#include "Misc/FileHelper.h"
#include "Misc/OutputDeviceFile.h"

#include "ShaderCompiler.h"
#include "TimerManager.h"

#include "ProfilingDebugging/CsvProfiler.h"
#include "ProfilingDebugging/MallocLeakReporter.h"
#include "ProfilingDebugging/TraceAuxiliary.h"

#include "StaticCameraTests/AutomatedPerfTestStaticCamera.h"
#include "Serialization/JsonSerializer.h"
#include "Serialization/JsonWriter.h"
#include "Serialization/JsonReader.h"
#include "Serialization/MemoryWriter.h"
#include "Serialization/MemoryReader.h"

#include "WorldPartition/WorldPartition.h"
#include "WorldPartition/WorldPartitionRuntimeCellDataSpatialHash.h"
#include "WorldPartition/WorldPartitionRuntimeHash.h"
#include "WorldPartition/WorldPartitionRuntimeSpatialHash.h"
#include "WorldPartition/WorldPartitionSubsystem.h"


// Incrementally stream in textures to smooth out memory spikes

#include UE_INLINE_GENERATED_CPP_BY_NAME(ProfileGo)
static int32 GStreamResourcesIterationCount = 15;

// Delta used for setting up the timer rate, and the delta to pass into the streaming system. 
// This is used to check if all streaming is completed.
static float GWaitOnStreamingTimeDelta = 0.1f;

UProfileGo::UProfileGo()
{
	CurrentState = EProfileGoStateAPT::None;
	TimeInState = 0.0f;
	DefaultSettleTime = 5.0f;
	LastPlatformTimeSeconds = 0.0f;
	bFirstTickInState = false;
	bWaitingOnCommandComplete = false;
	LastTeleportPlatformTimeSeconds = 0.0f;
	ResourceStreamingIteration = 0;
	bEncounteredError = false;
	ErrorMessage = FString("No error encountered.");
	DefaultSyncInterval = -1;
	DutyCycle = -1;
	SenarioStartTime = 0.0;
	DutyCycleTime = 0.0;

	StateTickers.Add(EProfileGoStateAPT::StartingRequest).BindUObject(this, &ThisClass::TickStartingRequest);
	StateTickers.Add(EProfileGoStateAPT::SettlingLocation).BindUObject(this, &ThisClass::TickSettling);
	StateTickers.Add(EProfileGoStateAPT::RunningCommands).BindUObject(this, &ThisClass::TickProfiling);
	StateTickers.Add(EProfileGoStateAPT::CompletedScenario).BindUObject(this, &ThisClass::TickCompletedScenario);
	StateTickers.Add(EProfileGoStateAPT::Summary).BindUObject(this, &ThisClass::TickSummary);
	StateTickers.Add(EProfileGoStateAPT::Completed).BindUObject(this, &ThisClass::TickCompleted);
}

bool UProfileGo::IsRunning() const
{
	return CurrentState != EProfileGoStateAPT::None;
}

void UProfileGo::Tick(float DeltaTime)
{
	// DeltaTime from FTSTicker is the time since the last game frame, not since the last tick the delegate received.
	// Since that DeltaTime is not useful, track our own
	const double PlatformTimeSeconds = FPlatformTime::Seconds();
	const float DeltaTickSeconds = PlatformTimeSeconds - LastPlatformTimeSeconds;
	LastPlatformTimeSeconds = PlatformTimeSeconds;

	if (!bFirstTickInState)
	{
		TimeInState += DeltaTickSeconds;
	}

	if (IsRunning())
	{
		const APlayerController* PlayerController = Cast<APlayerController>(GetPlayerController());
		if (!PlayerController)
		{
			return;
		}

		if (!CurrentRequest->bIgnorePawn)
		{
			const APawn* PlayerPawn = Cast<APawn>(PlayerController->GetPawn());
			if (!PlayerPawn && CurrentState == EProfileGoStateAPT::StartingRequest)
			{
				// Return until we spawn a pawn before beginning
				return;
			}
		}
	}

	EProfileGoStateAPT EntryState = CurrentState;

	if (StateTickers.Contains(EntryState))
	{
		StateTickers[EntryState].Execute(DeltaTickSeconds);
	}

	// still have the same state?
	if (CurrentState == EntryState)
	{
		bFirstTickInState = false;
	}
}

bool UProfileGo::Run(UProfileGoSubsystem* InSubsystem, const TCHAR* InProfileName, const TCHAR* InProfileArgs)
{
	FString InName = InProfileName;
	UE_LOGF(LogAutomatedPerfTest, Log, "UProfileGo::ProfileGo(UWorld*, InProfileName = \"%ls\", InProfileArgs = \"%ls\")", InProfileName, InProfileArgs);

	if (!InSubsystem)
	{
		UE_LOGFMT(LogAutomatedPerfTest, Log, "InSubsystem is nullptr! Aborting ProfileGo.");
		return false;
	}

	ProfileGoSubsystem = InSubsystem;
	if (FPaths::ValidatePath(InName) && FPaths::FileExists(InName))
	{
		// Try to replace the scenario string with the contents of the given file
		if (FFileHelper::LoadFileToString(InName, &FPlatformFileManager::Get().GetPlatformFile(), *InName))
		{
			UE_LOGF(LogAutomatedPerfTest, Log, "Read file at %ls to set profilego scenario: %ls", InProfileName, *InName);
			// Remove newline characters which are convenient to use when writing these files
			InName.ReplaceInline(TEXT("\n"), TEXT(""));
		}
		else
		{
			UE_LOGF(LogAutomatedPerfTest, Error, "Failed to read profilego scenario from file at %ls", InProfileName);
			return false;
		}
	}

	bFirstTickInState = true;
	TimeInState = 0.0f;

	if (InName.Equals(TEXT("save"), ESearchCase::IgnoreCase))
	{
		FProfileGoScenarioAPT NewScenario = CreateScenarioHere(InProfileArgs);
		FString AsText;
		NewScenario.StaticStruct()->ExportText(AsText, &NewScenario, nullptr, nullptr, PPF_ExportsNotFullyQualified | PPF_Copy | PPF_Delimited, nullptr);

		ClientMessage(*FString::Printf(TEXT("Saved %s"), *AsText));

		return true;
	}
	else if (InName.Equals(TEXT("stop"), ESearchCase::IgnoreCase))
	{
		SetState(EProfileGoStateAPT::Summary);
		return false;
	}

	CurrentState = EProfileGoStateAPT::None;

	CurrentRequest = MakeShareable(new FProfileGoRequestAPT);

	// save name and args
	CurrentRequest->RequestName = InProfileName;
	CurrentRequest->RequestArgs = InProfileArgs;

	// parse settle time
	CurrentRequest->SettleTime = DefaultSettleTime;
	FParse::Value(InProfileArgs, TEXT("-settle="), CurrentRequest->SettleTime);

	CurrentRequest->bSkipSettling = FParse::Param(*CurrentRequest->RequestArgs, TEXT("skipsettle"));
	CurrentRequest->bRetraceZ = FParse::Param(*CurrentRequest->RequestArgs, TEXT("retracez"));

	CurrentRequest->bIgnorePawn = FParse::Param(*CurrentRequest->RequestArgs, TEXT("ignorepawn"));

	TArray<FString> ProfileNames;
	InName.ParseIntoArray(ProfileNames, TEXT(","), true);
	NumRequestedScenarios = ProfileNames.Num();
	for (FString Name : ProfileNames)
	{
		// TODO Refactor this to find/generate the scenarios then add them to the request rather than adding them while generating (or finding them while adding)
		if (AddScenariosToCurrentRequest(*Name, false) == false)
		{
			if (TryGenerateScenariosForCurrentRequest(*Name, InProfileArgs) == false)
			{
				AbortWithError(*FString::Printf(TEXT("Could not find existing scenario named %s or generate new scenarios for %s %s"), *Name, *Name, InProfileArgs));
				ProfileGoSubsystem->OnRequestFailed().Broadcast();
				CurrentRequest = nullptr;
				return false;
			}
		}
	}

	// When running ProfileGo tests locally, it is too easy to accidentally give player input,
	//  such as moving the camera with the mouse, which can affect perf impact.
	//  Can avoid using -DisablePlayerInput for Gauntlet, or -profilego.disableplayerinput for client.
	const bool bDisablePlayerInput = FParse::Param(InProfileArgs, TEXT("disableplayerinput")) || FApp::IsUnattended();
	if (bDisablePlayerInput)
	{
		UWorld* World = GetWorld();
		check(World);

		APlayerController* PlayerController = World->GetFirstPlayerController();
		check(PlayerController);

		// Disable mouse input
		FInputModeUIOnly InputMode;
		InputMode.SetLockMouseToViewportBehavior(EMouseLockMode::DoNotLock);
		PlayerController->SetInputMode(InputMode);
		PlayerController->bShowMouseCursor = false;

		// Disable keyboard input
		PlayerController->SetIgnoreMoveInput(true);
		PlayerController->SetIgnoreLookInput(true);
	}

	RegisterInternalCommandDelegates();

	// If we have added all requested scenarios, begin the first scenario
	if (NumRequestedScenarios == 0 && CurrentRequest.IsValid())
	{
		SetState(EProfileGoStateAPT::StartingRequest);
		return true;
	}

	return false;
}

void UProfileGo::AddNewProfileGoCollection(FProfileGoCollectionAPT NewCollection)
{
	if (GetCollectionTemplates().ContainsByPredicate([NewCollection](FProfileGoCollectionAPT Collection) { return Collection.Name.Equals(NewCollection.Name); }))
	{
		UE_LOGF(LogAutomatedPerfTest, Log, "Collection named %ls already exists!", *NewCollection.Name);
		return;
	}

	TArray<FString> CollectionScenarios;
	CollectionScenarios = NewCollection.Scenarios;
	for (FString ScenarioName : CollectionScenarios)
	{
		if (!GetScenarioTemplates().ContainsByPredicate([ScenarioName](FProfileGoScenarioAPT Scenario) { return Scenario.Name.Equals(ScenarioName); }))
		{
			UE_LOGF(LogAutomatedPerfTest, Log, "Scenario named %ls does not exist! Cannot add a Collection containing an undefined Scenario. Use AddNewProfileGoScenario first.", *ScenarioName);
			return;
		}
	}

	GetCollectionTemplates().Add(NewCollection);
	UE_LOGF(LogAutomatedPerfTest, Log, "Added new collection named %ls", *NewCollection.Name);
}

void UProfileGo::AddNewProfileGoScenario(FProfileGoScenarioAPT NewScenario)
{
	if (GetScenarioTemplates().ContainsByPredicate([NewScenario](FProfileGoScenarioAPT Scenario) { return Scenario.Name.Equals(NewScenario.Name); }))
	{
		UE_LOGF(LogAutomatedPerfTest, Log, "Scenario named %ls already exists!", *NewScenario.Name);
		return;
	}

	GetScenarioTemplates().Add(NewScenario);
	UE_LOGF(LogAutomatedPerfTest, Log, "Added new scenario named %ls", *NewScenario.Name);
}

bool UProfileGo::AddScenariosToCurrentRequest(TArray<FProfileGoScenarioAPT> Scenarios, int InsertIndex /*= INDEX_NONE*/)
{
	if (InsertIndex < 0)
	{
		// By default, append the new scenarios to the end of the array
		CurrentRequest->PendingScenarios.Append(Scenarios);
	}
	else
	{
		// If an InsertIndex is specified, insert the scenarios into the array starting at that index
		CurrentRequest->PendingScenarios.Insert(Scenarios, InsertIndex);
	}

	NumRequestedScenarios--;
	if (NumRequestedScenarios < 0)
	{
		AbortWithError(TEXT("Tried to add more scenarios or collections than were requested. This should never happen. Aborting profilego!"));
		return false;
	}

	return true;
}

bool UProfileGo::AddScenariosToCurrentRequest(const TCHAR* ProfileName, bool bErrorIfNotFound /*= false*/, int InsertIndex /*= INDEX_NONE*/)
{
	if (!CurrentRequest.IsValid())
	{
		AbortWithError(TEXT("Current ProfileGo request is invalid. Aborting profilego!"));
		return false;
	}

	if (IsRunning())
	{
		ClientMessage(TEXT("Cannot add scenarios to a request that is already running! Use \"profilego stop\" to end an in-progress profilego run."));
		return false;
	}

	TArray<FProfileGoScenarioAPT> FoundScenarios;
	if (TryFindScenarios(ProfileName, FoundScenarios))
	{
		return AddScenariosToCurrentRequest(FoundScenarios, InsertIndex);
	}
	else if (bErrorIfNotFound)
	{
		FString AbortMessage = FString::Printf(TEXT("Could not find expected scenario or collection named %s. Aborting profilego!"), ProfileName);
		AbortWithError(*AbortMessage);
	}

	return false;
}

bool UProfileGo::TryFindScenarios(const TCHAR* ProfileName, TArray<FProfileGoScenarioAPT>& OutScenarios)
{
	OutScenarios = TArray<FProfileGoScenarioAPT>();
	TArray<FString> ScenarioNames = TArray<FString>();

	const FProfileGoCollectionAPT* CollectionExists = GetCollectionTemplates().FindByPredicate([ProfileName](const FProfileGoCollectionAPT& Collection)
	{
		return Collection.Name == ProfileName;
	});

	if (CollectionExists)
	{
		ScenarioNames = CollectionExists->Scenarios;
	}
	else
	{
		ScenarioNames.Add(ProfileName);
	}

	for (FString ScenarioName : ScenarioNames)
	{
		FString TrimmedScenarioName = ScenarioName.TrimStartAndEnd();
		const FProfileGoScenarioAPT* ScenarioExists = GetScenarioTemplates().FindByPredicate([TrimmedScenarioName](const FProfileGoScenarioAPT& Scenario)
			{
				return Scenario.Name == TrimmedScenarioName;
			});

		if (ScenarioExists)
		{
			OutScenarios.Add(FProfileGoScenarioAPT(*ScenarioExists));
		}
		else
		{
			UE_LOGF(LogAutomatedPerfTest, Log, "Scenario named %ls does not exist! Skipping.", *TrimmedScenarioName);
		}
	}

	return (OutScenarios.Num() > 0);
}


bool UProfileGo::WaitForLoadingAndStreaming(UWorld* InWorld, float DeltaTime, bool bIncrementalResourceStreaming /*= false*/)
{
	check(InWorld);

	if (bIncrementalResourceStreaming && bFirstTickInState)
	{
		ResourceStreamingIteration = 0;
	}

	// This will sync load levelstreaming which means the settle time is just a backup since loading should be completed
	GEngine->BlockTillLevelStreamingCompleted(InWorld);

	// This will block until all texture streaming is completed. Set sync state to ensure everything is loaded before finishing.
	bool bResourceStreamingCompleted = true;
	if (FStreamingManagerCollection* StreamingManagers = IStreamingManager::Get_Concurrent())
	{
		// If we're doing the final frame, do a complete update
		bool bUpdateAllResources = !bIncrementalResourceStreaming || (ResourceStreamingIteration >= GStreamResourcesIterationCount - 1);
		StreamingManagers->UpdateResourceStreaming(DeltaTime, bUpdateAllResources);

		// If we're doing an incremental update, we do a short 2ms wait to get the current count and tick streamers, otherwise we'll block for up to 5s
		const float StreamingBlockTimeLimit = bUpdateAllResources ? 5.0f : 0.02f;
		int32 NumUnfinishedRequests = StreamingManagers->BlockTillAllRequestsFinished(StreamingBlockTimeLimit, false);

		if (NumUnfinishedRequests > 0)
		{
			UE_LOGF(LogAutomatedPerfTest, Log, "TickSettling: Texture streaming did not finish %i requests within %ls seconds. Retrying settle.", NumUnfinishedRequests, *FString::SanitizeFloat(StreamingBlockTimeLimit));
			return true;
		}
		// Only when we've completed the final full update of all resources can we consider resource streaming "done"
		bResourceStreamingCompleted = bUpdateAllResources;
	}

	bool bLoadInProgress = false;
	if (IsAsyncLoading())
	{
		bLoadInProgress = true;
		UE_LOGFMT(LogAutomatedPerfTest, Log, "TickSettling: Waiting On AsyncLoading");
	}

	TWeakObjectPtr<UWorldPartitionSubsystem> WorldPartition = UWorld::GetSubsystem<UWorldPartitionSubsystem>(InWorld);
	// Ensure world partition streaming completes. Should be redundant with above IsStreamingInLevels()
	if (WorldPartition.Get() && !WorldPartition->IsAllStreamingCompleted())
	{
		UE_LOGFMT(LogAutomatedPerfTest, Log, "TickSettling: Waiting On World Partition Level Streaming");
		bLoadInProgress = true;
	}

	// Finish any outstanding shader jobs
	if (GShaderCompilingManager && GShaderCompilingManager->GetNumOutstandingJobs())
	{
		GShaderCompilingManager->FinishAllCompilation();
		UE_LOGFMT(LogAutomatedPerfTest, Log, "TickSettling: Waiting On Shaders");
		bLoadInProgress = true;
	}

	if (bIncrementalResourceStreaming && !bLoadInProgress)
	{
		// We're not done until we've completed the final resource streaming iteration
		ResourceStreamingIteration++;
		if (!bResourceStreamingCompleted)
		{
			UE_LOGFMT(LogAutomatedPerfTest, Log, "TickSettling: Waiting On Resource Streaming Final update");
			bLoadInProgress = true;
		}
	}

	return bLoadInProgress;
}

bool UProfileGo::HasEncounteredError() const
{
	return bEncounteredError;
}

FString UProfileGo::GetCurrentScenarioName() const
{
	if (IsRunning() == false)
	{
		return FString();
	}

	return CurrentRequest->CurrentResult.Scenario.Name;
}

void UProfileGo::LoadFromJSON(const FString& Filename)
{
	if (!FPaths::FileExists(Filename))
	{
		UE_LOGF(LogAutomatedPerfTest, Warning, "ProfileGo::LoadFromJSON: '%ls' not found.", *Filename);
		return;
	}

	FString JsonString;
	FFileHelper::LoadFileToString(JsonString, *Filename);
	TSharedRef<TJsonReader<>> Reader = TJsonReaderFactory<>::Create(JsonString);
	TSharedPtr<FJsonObject> JsonPreset = MakeShared<FJsonObject>();

	TArray<FProfileGoScenarioAPT>& Scenarios = GetScenarioTemplates();
	TArray<FProfileGoCollectionAPT>& Collections = GetCollectionTemplates();
	TArray<FProfileGoCommandAPT>& Commands = GetCommandTemplates();

	Scenarios.Empty();
	Collections.Empty();
	Commands.Empty();

	if (!FJsonSerializer::Deserialize(Reader, JsonPreset))
	{
		return;
	}

	// TODO: Add support for generated scenarios
	const auto ScenariosJson = JsonPreset->GetArrayField(TEXT("Scenarios"));
	const auto CollectionsJson = JsonPreset->GetArrayField(TEXT("Collections"));
	const auto CommandsJson = JsonPreset->GetArrayField(TEXT("Commands"));

	FJsonObjectConverter::JsonArrayToUStruct(ScenariosJson, &Scenarios);
	FJsonObjectConverter::JsonArrayToUStruct(CollectionsJson, &Collections);
	FJsonObjectConverter::JsonArrayToUStruct(CommandsJson, &Commands);

	UpdateProjectSettings();
}

void UProfileGo::SaveToJSON(const FString& Filename)
{
	FString OutputString;
	TSharedRef<TJsonWriter<>> Writer = TJsonWriterFactory<>::Create(&OutputString);
	const auto WriteArray = [Writer]<typename TStruct>(const TArray<TStruct>&ObjectArray, FString ObjectName)
	{
		FString JsonString;
		Writer->WriteArrayStart(ObjectName);
		for (const TStruct& Item : ObjectArray)
		{
			FJsonObjectConverter::UStructToJsonObjectString(TStruct::StaticStruct(), &Item, JsonString, 0, 0);
			Writer->WriteRawJSONValue(*JsonString);
		}
		Writer->WriteArrayEnd();
	};

	if (const UAutomatedProfileGoTestProjectSettings* Settings = GetDefault<UAutomatedProfileGoTestProjectSettings>())
	{
		// TODO: Add support for generated scenarios
		Writer->WriteObjectStart();
		WriteArray(Settings->Scenarios, "Scenarios");
		WriteArray(Settings->Collections, "Collections");
		WriteArray(Settings->Commands, "Commands");
		Writer->WriteObjectEnd();
		Writer->Close();

		FFileHelper::SaveStringToFile(OutputString, *Filename);
	}
}

FString UProfileGo::GetStatus() const
{
	if (HasEncounteredError())
	{
		return ErrorMessage;
	}

	if (IsRunning() == false)
	{
		return TEXT("Not Running");
	}

	const int32 Completed = CurrentRequest->ScenarioResults.Num();
	const int32 Total = Completed + CurrentRequest->PendingScenarios.Num();

	return FString::Printf(TEXT("Profiling %s (stage %s) %d/%d)"),
		*GetCurrentScenarioName(), *GetStateName(CurrentState), Completed, Total);
}

void UProfileGo::TickStartingRequest(float DeltaTime)
{
	check(CurrentState == EProfileGoStateAPT::StartingRequest);

	if (!CurrentRequest.IsValid() || CurrentRequest->PendingScenarios.Num() < 1)
	{
		AbortWithError(TEXT("Invalid request!"));
		return;
	}

	if (bFirstTickInState)
	{
		CSV_EVENT(AutomatedPerfTest, TEXT("StartingRequest: %s"), *CurrentRequest->RequestName);

		if (CurrentRequest->ScenarioResults.Num() == 0)
		{
			ProfileGoSubsystem->OnPassStarted().Broadcast();
		}

		// parse run first commands
		FString RunFirstCommands;
		TryParseValue(*CurrentRequest->RequestArgs, TEXT("-runfirst="), RunFirstCommands);
		CurrentRequest->PendingCommands.Insert(CreateCommandList(RunFirstCommands), 0);
	}

	// Wait for the settle time once before beginning the request
	if (TimeInState < CurrentRequest->SettleTime)
	{
		return;
	}

	// Execute RunFirst commands before beginning first scenario
	if (TryExecuteNextPendingCommand())
	{
		return;
	}

	ClientMessage(*FString::Printf(TEXT("Finished setting up request: %s. Starting profile of (%d scenarios)"), *CurrentRequest->RequestName, CurrentRequest->PendingScenarios.Num()));
	StartNextScenario();
}

void UProfileGo::TickSettling(float DeltaTime)
{
	check(CurrentState == EProfileGoStateAPT::SettlingLocation);

	if (bFirstTickInState)
	{
		CSV_EVENT_GLOBAL(TEXT("ProfileGo Settling"));
	}

	FProfileGoScenarioResultAPT& CurrentResult = CurrentRequest->CurrentResult;

	// Tries to force all streaming sources and return true if we are waiting on any related system to finish loading
	if (WaitForLoadingAndStreaming(GetWorld(), DeltaTime, true))
	{
		CurrentResult.AsyncLoadTime = TimeInState;
		return;
	}

	// Wait an additional amount of specified time
	const double NonAsyncLoadSettleTime = (TimeInState - CurrentResult.AsyncLoadTime);
	if (NonAsyncLoadSettleTime < CurrentRequest->SettleTime
		|| NonAsyncLoadSettleTime < DutyCycleTime)
	{
		return;
	}

	// Ensure we make it to our intended location
	if (!CurrentResult.bArrivedAtLocation)
	{
		CurrentResult.bArrivedAtLocation = TickSettlingLocation(DeltaTime);
		return;
	}

	// Garbage Collect
	if (!CurrentResult.bRanGC)
	{
		if (GEngine)
		{
			GEngine->TrimMemory();
			CurrentResult.bRanGC = true;
		}
		return;
	}

	ClientMessage(*FString::Printf(TEXT("Settled and finished async load in %.02f seconds (%.02f seconds for duty cycle %d%%)"), TimeInState, DutyCycleTime, DutyCycle));
	SetState(EProfileGoStateAPT::RunningCommands);
}

void UProfileGo::TickProfiling(float DeltaTime)
{
	check(CurrentState == EProfileGoStateAPT::RunningCommands);

	FProfileGoScenarioResultAPT& CurrentResult = CurrentRequest->CurrentResult;
	const FProfileGoScenarioAPT* CurrentScenario = &CurrentResult.Scenario;

	if (bFirstTickInState)
	{
		CSV_EVENT_GLOBAL(TEXT("ProfileGo RunningCommands"));

		// Add OnBegin commands, but do not override
		FString OnBeginCommandString = CurrentScenario->OnBegin;
		if (OnBeginCommandString.IsEmpty())
		{
			TryParseValue(*CurrentRequest->RequestArgs, TEXT("-onbegin="), OnBeginCommandString);
		}
		CurrentRequest->PendingCommands.Append(CreateCommandList(OnBeginCommandString));

		// Add Run commands
		if (!CurrentScenario->OverrideCommands.IsEmpty())
		{
			CurrentRequest->PendingCommands.Append(CreateCommandList(CurrentScenario->OverrideCommands));
		}
		else if (!CurrentScenario->SkipRunCommands)
		{
			FString CommandsArg;
			TryParseValue(*CurrentRequest->RequestArgs, TEXT("-run="), CommandsArg);
			CurrentRequest->PendingCommands.Append(CreateCommandList(CommandsArg));
		}

		// Add OnEnd commands, but do not override
		FString OnEndCommandString = CurrentScenario->OnEnd;
		if (OnEndCommandString.IsEmpty())
		{
			TryParseValue(*CurrentRequest->RequestArgs, TEXT("-onend="), OnEndCommandString);
		}
		CurrentRequest->PendingCommands.Append(CreateCommandList(OnEndCommandString));

		return;
	}

	// when firing commands with waits we set this value to a negative
	if (TimeInState < 0)
	{
		return;
	}

	if (!TryExecuteNextPendingCommand())
	{
		ClientMessage(*FString::Printf(TEXT("Finished profiling %s"), *CurrentScenario->Name));
		SetState(EProfileGoStateAPT::CompletedScenario);
	}
}

void UProfileGo::TickCompletedScenario(float DeltaTime)
{
	check(CurrentState == EProfileGoStateAPT::CompletedScenario);

	FProfileGoScenarioResultAPT& CurrentResult = CurrentRequest->CurrentResult;
	if (CurrentResult.bCompleted == false)
	{
		CurrentResult.bCompleted = true;

		CurrentRequest->ScenarioResults.Add(CurrentResult);

		CSV_EVENT_GLOBAL(TEXT("ProfileGo CompletedScenario: %s"), *CurrentResult.Scenario.Name);
		TRACE_END_REGION(*WriteToString<256>(TEXT("ProfileGo: "), *CurrentResult.Scenario.Name));

		ProfileGoSubsystem->OnScenarioEnded().Broadcast(CurrentResult.Scenario.Name);

		//AdjustCVarsForScenarioEnd();

		const int TotalScenarios = CurrentRequest->ScenarioResults.Num() + CurrentRequest->PendingScenarios.Num();
		UE_LOGF(LogAutomatedPerfTest, Log, "Completed Scenario %ls. (%d/%d)", *CurrentResult.Scenario.Name, CurrentRequest->ScenarioResults.Num(), TotalScenarios);
		return;
	}

	// move on or done
	if (CurrentRequest->PendingScenarios.Num())
	{
		SetState(EProfileGoStateAPT::None);
		StartNextScenario();
	}
	else
	{
		SetState(EProfileGoStateAPT::Summary);
	}
}

void UProfileGo::TickSummary(float DeltaTime)
{
	if (bFirstTickInState)
	{
		// parse RunLast commands
		FString RunLastCommands;
		TryParseValue(*CurrentRequest->RequestArgs, TEXT("-runlast="), RunLastCommands);
		CurrentRequest->PendingCommands.Append(CreateCommandList(RunLastCommands));

		if (CurrentRequest->PendingCommands.Num())
		{
			CSV_EVENT(AutomatedPerfTest, TEXT("RunLast Commands"));
			ClientMessage(*FString::Printf(TEXT("Executing RunLast commands for %s"), *CurrentRequest->RequestName));
		}
	}

	if (TimeInState < 0)
	{
		return;
	}

	if (TryExecuteNextPendingCommand())
	{
		return;
	}

	CSV_EVENT_GLOBAL(TEXT("ProfileGo Summary"));

	SetState(EProfileGoStateAPT::Completed);
}

void UProfileGo::TickCompleted(float DeltaTime)
{
	CSV_EVENT_GLOBAL(TEXT("ProfileGo Complete"));
	ProfileGoSubsystem->OnPassEnded().Broadcast();
	SetState(EProfileGoStateAPT::None);
}

void UProfileGo::StartNextScenario()
{
	// New result container
	CurrentRequest->CurrentResult = FProfileGoScenarioResultAPT();
	CurrentRequest->CurrentResult.Scenario = CurrentRequest->PendingScenarios[0];
	// Once read, remove from the queue
	CurrentRequest->PendingScenarios.RemoveAt(0);

	FProfileGoScenarioAPT CurrentScenario = CurrentRequest->CurrentResult.Scenario;
	CSV_EVENT(AutomatedPerfTest, TEXT("StartScenario: %s"), *CurrentScenario.Name);
	TRACE_BEGIN_REGION(*WriteToString<256>(TEXT("ProfileGo: "), *CurrentScenario.Name));
	ClientMessage(*FString::Printf(TEXT("Starting Scenario %s"), *CurrentScenario.Name));

	if (!CurrentScenario.Position.IsZero() || !CurrentScenario.Orientation.IsZero())
	{
		TeleportToScenario(CurrentScenario);
	}
	else
	{
		CurrentRequest->CurrentResult.bArrivedAtLocation = true;
	}

	if (CurrentRequest->bSkipSettling)
	{
		SetState(EProfileGoStateAPT::RunningCommands);
	}
	else
	{
		SetState(EProfileGoStateAPT::SettlingLocation);
	}

	ProfileGoSubsystem->OnScenarioStarted().Broadcast(CurrentScenario.Name);
}

void UProfileGo::TeleportToScenario(FProfileGoScenarioAPT& CurrentScenario)
{
	ClientMessage(*FString::Printf(TEXT("Teleport to scenario %s"), *CurrentScenario.Name));

	FVector Pos = CurrentScenario.Position;
	FRotator Ori = CurrentScenario.Orientation;

	const TCHAR* CheatCmd = TEXT("cheat BugItGo");

	FString Cmd = FString::Printf(TEXT("%s %.04f %.04f %.04f %.04f %.04f %.04f"),
		CheatCmd,
		Pos.X, Pos.Y, Pos.Z,
		Ori.Pitch, Ori.Yaw, Ori.Roll);

	ConsoleCommand(*Cmd);
	LastTeleportPlatformTimeSeconds = FPlatformTime::Seconds();
}

bool UProfileGo::TickSettlingLocation(float DeltaTime)
{
	if (APlayerController* PlayerController = Cast<APlayerController>(GetPlayerController()))
	{
		if (APawn* Pawn = Cast<APawn>(PlayerController->GetPawn()))
		{
			FProfileGoScenarioResultAPT& CurrentResult = CurrentRequest->CurrentResult;

			// Wait until the pawn stops moving
			static const float MaxSettleSecondsForVelocityWait = 10.0f;
			if (TimeInState < MaxSettleSecondsForVelocityWait
				&& !Pawn->GetVelocity().IsNearlyZero(1.0))
			{
				return false;
			}

			// Check if we are close enough to our target position. If not, retry teleport just once.
			FVector CurrentLocation = Pawn->GetActorLocation();
			FRotator CurrentRotation = Pawn->GetViewRotation();

			FVector TargetLocation = CurrentResult.Scenario.Position;
			FRotator TargetRotation = CurrentResult.Scenario.Orientation;
			const double XYTolerance = 1.0;
			const double ZTolerance = 200.0;
			CurrentResult.bArrivedAtLocation = FMath::IsNearlyEqual(CurrentLocation.X, TargetLocation.X, XYTolerance)
				&& FMath::IsNearlyEqual(CurrentLocation.Y, TargetLocation.Y, XYTolerance)
				&& FMath::IsNearlyEqual(CurrentLocation.Z, TargetLocation.Z, ZTolerance);

			bool bNeedRetrace = CurrentRequest->bRetraceZ && !CurrentResult.bRetracedZ;
			if (CurrentResult.bArrivedAtLocation && !bNeedRetrace)
			{
				ClientMessage(*FString::Printf(TEXT("Arrived at location for scenario %s"), *CurrentResult.Scenario.Name));
				return true;
			}

			// Always retry teleporting once if we didn't end up close enough to our target location
			// After the second failed retry just skip it with message in logs - retraceZ is not needed
			if (!CurrentResult.bArrivedAtLocation)
			{
				// wait not longer than settle time
				if (FPlatformTime::Seconds() - LastTeleportPlatformTimeSeconds < CurrentRequest->SettleTime)
				{
					return false;
				}

				if (CurrentResult.bRetriedTeleport)
				{
					ClientMessage(*FString::Printf(TEXT("Character did not reach target transform for scenario %s. Adjusted location and rotation from %s %s to %s %s"),
						*CurrentResult.Scenario.Name,
						*TargetLocation.ToCompactString(), *TargetRotation.ToCompactString(),
						*CurrentLocation.ToCompactString(), *CurrentRotation.ToCompactString()));
					CurrentResult.Scenario.Position = CurrentLocation;
					CurrentResult.Scenario.Orientation = CurrentRotation;
					CurrentResult.bArrivedAtLocation = true;
					return true;
				}
				else
				{
					ClientMessage(*FString::Printf(TEXT("Retrying teleport for scenario %s"), *CurrentResult.Scenario.Name));
					TeleportToScenario(CurrentResult.Scenario);
					CurrentResult.bRetriedTeleport = true;
					return false;
				}
			}

			// Retrace Z attempt after character arrived at location
			ClientMessage(*FString::Printf(TEXT("Attempting retrace Z for scenario %s"), *CurrentResult.Scenario.Name));
			// Force setting a massive height so raycast hits ground
			// TODO: refine how this is set correctly for world volume height instead of a magic number, see 'wp:'
			FVector NewScenarioPosition = TargetLocation;
			NewScenarioPosition.Z = 1000000.0f;
			if (FindBestStandingPoint(GetWorld(), NewScenarioPosition, NewScenarioPosition))
			{
				ClientMessage(*FString::Printf(TEXT("Adjusted Z from %f to %f"),
					TargetLocation.Z, NewScenarioPosition.Z));
				CurrentResult.Scenario.Position = NewScenarioPosition;
				TeleportToScenario(CurrentResult.Scenario);
			}
			else
			{
				ClientMessage(*FString::Printf(TEXT("Failed to adjust Z from %f"), TargetLocation.Z));
			}
			CurrentResult.bRetracedZ = true;
			return false;
		}
		else if (CurrentRequest->bIgnorePawn)
		{
			return true;
		}
	}

	return false;
}

void UProfileGo::AbortWithError(const TCHAR* InErrorMessage)
{
	ClientMessage(*FString::Printf(TEXT("Abort with error: %s"), InErrorMessage));
	bEncounteredError = true;
	ErrorMessage = InErrorMessage;
	SetState(EProfileGoStateAPT::None);
}

void UProfileGo::ConsoleCommand(const TCHAR* InCommand)
{
	if (APlayerController* Controller = GetPlayerController())
	{
		Controller->ConsoleCommand(InCommand);
	}
}

void UProfileGo::ClientMessage(const TCHAR* InMsg)
{
	if (APlayerController* Controller = GetPlayerController())
	{
		Controller->ClientMessage(InMsg);
	}

	UE_LOGF(LogAutomatedPerfTest, Log, "%ls", InMsg);
}

void UProfileGo::StartCommandLog(const FProfileGoCommandAPT& ProfileGoCommand)
{
	if (ProfileGoCommand.Log.IsEmpty())
	{
		return;
	}

	StopCommandLog();

	FString LogName = FString::Printf(TEXT("%s_%s"), *CurrentRequest->CurrentResult.Scenario.Name, *ProfileGoCommand.Log);
	// Remove any leftover invalid characters such as colons that can cause invalid file names
	LogName = FPaths::MakeValidFileName(LogName);

	FString ProfilingDirectory = *FPaths::ProfilingDir();

	const bool IsCSV = FParse::Param(*ProfileGoCommand.Command, TEXT("CSV"));
	const FString Extension = IsCSV ? TEXT("csv") : TEXT("txt");

	FString FileName = FString::Printf(TEXT("%s/profilego/%s.%s"),
		*ProfilingDirectory,
		*LogName, *Extension);

	bool bShouldAppend = PreviousLogs.Contains(LogName);
	ResultsFile = MakeShareable(new FOutputDeviceFile(*FileName, true, bShouldAppend));
	ResultsFile->SetSuppressEventTag(true);
	if (ProfileGoCommand.CopyOutputToGameLog)
	{
		FOutputDeviceRedirector::Get()->AddOutputDevice(ResultsFile.Get());
	}
	PreviousLogs.Add(LogName);
}

void UProfileGo::StopCommandLog()
{
	if (ResultsFile.IsValid())
	{
		if (FOutputDeviceRedirector::Get()->IsRedirectingTo(ResultsFile.Get()))
		{
			FOutputDeviceRedirector::Get()->RemoveOutputDevice(ResultsFile.Get());
		}
		ResultsFile = nullptr;
	}
}

bool UProfileGo::TryGenerateScenariosForCurrentRequest(const TCHAR* InProfileName, const TCHAR* InProfileArgs)
{
	FString ProfileName = InProfileName;
	FString ScenarioCommand = InProfileName;
	FString ProfileArgs = InProfileArgs;

	// Check if we have a matching GeneratedScenario defined in our config
	const FProfileGoGeneratedScenarioAPT* GeneratedScenarioExists = ProfileGoGeneratedScenarios.FindByPredicate(
	[ProfileName](const FProfileGoGeneratedScenarioAPT& GeneratedScenario)
	{
		return GeneratedScenario.Name.Equals(ProfileName, ESearchCase::IgnoreCase);
	});

	if (GeneratedScenarioExists)
	{
		// Either our GeneratedSecnario is incomplete and invalid or we are wrapping another config scenario with CopyLocation
		if (GeneratedScenarioExists->ScenarioCommand.IsEmpty())
		{
			FString TrimmedScenarioName = GeneratedScenarioExists->CopyLocation.TrimStartAndEnd();
			if (!TrimmedScenarioName.IsEmpty())
			{
				const FProfileGoScenarioAPT* ScenarioExists = GetScenarioTemplates().FindByPredicate([TrimmedScenarioName](const FProfileGoScenarioAPT& Scenario)
					{
						return Scenario.Name == TrimmedScenarioName;
					});

				if (ScenarioExists)
				{
					FProfileGoScenarioAPT NewProfileScenario;
					NewProfileScenario.Name = ProfileName;
					NewProfileScenario.Position = ScenarioExists->Position;
					NewProfileScenario.Orientation = ScenarioExists->Orientation;
					NewProfileScenario.AutoGenerated = true;
					NewProfileScenario.OnBegin = GeneratedScenarioExists->OnBegin;
					NewProfileScenario.OverrideCommands = GeneratedScenarioExists->OverrideCommands;
					NewProfileScenario.OnEnd = GeneratedScenarioExists->OnEnd;
					NewProfileScenario.SkipRunCommands = GeneratedScenarioExists->SkipRunCommands;

					AddNewProfileGoScenario(NewProfileScenario);
					return AddScenariosToCurrentRequest(*ProfileName);
				}

				ClientMessage(*FString::Printf(TEXT("CopyLocation is invalid for generated scenario named %s!"), *ProfileName));
				return false;
			}
			else
			{
				ClientMessage(*FString::Printf(TEXT("Generated scenario named %s must include a value for ScenarioCommand or CopyLocation!"), *ProfileName));
				return false;
			}
		}

		// If we are not using CopyLocation to wrap an existing scenario, then append it to ScenarioCommand to be parsed below
		FString GeneratedScenarioCommand = GeneratedScenarioExists->ScenarioCommand;
		if (!GeneratedScenarioExists->CopyLocation.IsEmpty())
		{
			GeneratedScenarioCommand += "@" + GeneratedScenarioExists->CopyLocation;
		}

		// If a scenario exists with this ProfileName, make sure to distinguish between name and command (eg. "DefaultGrid" and "grid:16x16")
		ScenarioCommand = GeneratedScenarioCommand;

		if (!ProfileArgs.IsEmpty() && !GeneratedScenarioExists->Arguments.IsEmpty())
		{
			// Join additional args with a space
			ProfileArgs.Append(TEXT(" "));
		}
		ProfileArgs.Append(GeneratedScenarioExists->Arguments);
	}

	ClientMessage(*FString::Printf(TEXT("Attempting to generate scenarios for command %s with args %s"), *ScenarioCommand, *ProfileArgs));

	const auto FindAndExecuteHandler = [&](bool& bOutCommandResult)
	{
		TPair<FString, GeneratedScenarioHandlerDelegate>* Found = Algo::FindByPredicate(GenerateScenarioHandlers,
		[ScenarioCommand](const TPair<FString, GeneratedScenarioHandlerDelegate>& Element)
		{
			return ScenarioCommand.StartsWith(Element.Key);
		});
		
		bOutCommandResult = false;
		if (Found != nullptr)
		{
			UE_LOGF(LogAutomatedPerfTest, Verbose, "ProfileGo: Using command handler '%ls' to generate scenario '%ls'", *Found->Key, *ScenarioCommand);
		    // ExecuteIfBound not supported for delegates with return type
			bOutCommandResult = Found->Value.IsBound() ? Found->Value.Execute(ScenarioCommand, ProfileArgs) : false;
			return true;
		}

		return false;
	};

	bool bCommandResult = false;
	if (FindAndExecuteHandler(bCommandResult))
	{
		return bCommandResult;
	}

	ClientMessage(*FString::Printf(TEXT("Failed to generate scenarios for %s %s"), *ProfileName, *ProfileArgs));
	return false;
}

bool UProfileGo::TryParseValue(const TCHAR* Stream, const TCHAR* ParamName, FString& OutValue, bool bShouldStopOnSeparator /*= false*/)
{
	bool bSuccess = FParse::Value(Stream, ParamName, OutValue, bShouldStopOnSeparator);

	// If the value is a valid file path, try to read from that file and replace OutValue
	if (bSuccess && FPaths::ValidatePath(OutValue) && FPaths::FileExists(OutValue))
	{
		if (FFileHelper::LoadFileToString(OutValue, &FPlatformFileManager::Get().GetPlatformFile(), *OutValue))
		{
			// Remove newline characters which are convenient to use when writing these files
			OutValue.ReplaceInline(TEXT("\n"), TEXT(""));
			bSuccess = true;
		}
		else
		{
			UE_LOGF(LogAutomatedPerfTest, Error, "Failed to read profilego param %ls from file at %ls", ParamName, *OutValue);
		}
	}

	return bSuccess;
}

void UProfileGo::ExecuteProfileGoCommand(const FProfileGoCommandAPT& ProfileGoCommand)
{
	UE_LOGF(LogAutomatedPerfTest, Log, "Executing command: %ls", *ProfileGoCommand.Command);
	StartCommandLog(ProfileGoCommand);

	// Block further commands until receiving a callback to signal the command has finished or timeout after a max wait
	if (ProfileGoCommand.WaitForCallback)
	{
		// 10 seconds is our default max wait if we don't specify a wait
		float MaxWaitSeconds = ProfileGoCommand.Wait > 0.f ? ProfileGoCommand.Wait : 10.f;
		WaitForCommandComplete(MaxWaitSeconds);
	}
	else
	{
		// If we aren't waiting for a callback, wait for at least this long before moving to the next command
		TimeInState = -ProfileGoCommand.Wait;
	}

	if (TryHandleProfileGoCommandString(ProfileGoCommand) == false)
	{
		// regular console command
#if PLATFORM_IOS
		if (ProfileGoCommand.Command.Contains(TEXT("profilegpu")) == false && ProfileGoCommand.Command.Contains(TEXT("screenshot")) == false)
#endif
		{
			if (!ProfileGoCommand.CopyOutputToGameLog && ResultsFile)
			{
				GEngine->Exec(GetWorld(), *ProfileGoCommand.Command, *ResultsFile);
			}
			else
			{
				ConsoleCommand(*ProfileGoCommand.Command);
			}
		}
	}
}

void UProfileGo::MarkCommandComplete()
{
	bWaitingOnCommandComplete = false;
	GetWorld()->GetTimerManager().ClearTimer(CommandCompleteFallbackTimerHandle);
}

void UProfileGo::CheckIfStreamingCompleted()
{
	// Tries to force all streaming sources and return true if we are waiting on any related system to finish loading
	if (!WaitForLoadingAndStreaming(GetWorld(), GWaitOnStreamingTimeDelta, true))
	{
		bWaitingOnCommandComplete = false;
		return;
	}

	// Re-wait since streaming isn't finished.
	WaitForStreamingCommandComplete();
}

void UProfileGo::WaitForCommandComplete(float MaxWaitSeconds /*= 10.f*/)
{
	bWaitingOnCommandComplete = true;
	GetWorld()->GetTimerManager().SetTimer(CommandCompleteFallbackTimerHandle, this, &UProfileGo::MarkCommandComplete, 0.f, false, MaxWaitSeconds);
}

void UProfileGo::WaitForStreamingCommandComplete()
{
	bWaitingOnCommandComplete = true;
	GetWorld()->GetTimerManager().SetTimer(CommandCompleteFallbackTimerHandle, this, &UProfileGo::CheckIfStreamingCompleted, GWaitOnStreamingTimeDelta, false);
}

bool UProfileGo::TryHandleProfileGoCommandString(const FProfileGoCommandAPT& ProfileGoCommand)
{
	FProfileGoScenarioResultAPT& CurrentResult = CurrentRequest->CurrentResult;

	FString CommandString = ProfileGoCommand.Command;
	FString ScenarioName = FPaths::MakeValidFileName(CurrentResult.Scenario.Name);
	ScenarioName = ScenarioName.Replace(TEXT(" "), TEXT("_"));
	CommandString.ReplaceInline(TEXT("{Location}"), *ScenarioName);

	const_cast<FProfileGoCommandAPT&>(ProfileGoCommand).Command = CommandString;

	const auto FindAndExecuteHandler = [&](bool& bOutCommandResult)
	{
		TPair<FString, CommandHandlerDelegate>* Found = Algo::FindByPredicate(CommandHandlers, 
		[CommandString](const TPair<FString, CommandHandlerDelegate>& Element)
		{
			return CommandString.StartsWith(Element.Key);
		});
		
		bOutCommandResult = false;
		if (Found != nullptr)
		{
			UE_LOGF(LogAutomatedPerfTest, Verbose, "ProfileGo: Using command handler for '%ls' to run '%ls'", *Found->Key, *CommandString);
		    // ExecuteIfBound not supported for delegates with return type
			bOutCommandResult = Found->Value.IsBound() ? Found->Value.Execute(CommandString, ProfileGoCommand) : false;
			return true;
		}

		return false;
	};

	bool bCommandResult = false;
	if (FindAndExecuteHandler(bCommandResult))
	{
		return bCommandResult;
	}

	return false;
}

void UProfileGo::RegisterCommandDelegate(const FString& Command, CommandHandlerDelegate&& Handler)
{
    if(!CommandHandlers.Contains(Command))
	{
		CommandHandlers.Add(Command, Handler);
	}
}

void UProfileGo::RegisterGeneratedScenarioDelegate(const FString& Command, GeneratedScenarioHandlerDelegate&& Handler)
{
	if (!GenerateScenarioHandlers.Contains(Command))
	{
		GenerateScenarioHandlers.Add(Command, Handler);
	}
}

void UProfileGo::RegisterInternalCommandDelegates()
{
	RegisterCommandDelegate(TEXT("begin csvprofile"), 
	CommandHandlerDelegate::CreateLambda([this](FString& CommandString, const FProfileGoCommandAPT& ProfileGoCommand) -> bool
	{
#if CSV_PROFILER
		FCsvProfiler* const CsvProfiler = FCsvProfiler::Get();
		if (!CsvProfiler)
		{
			return false;
		}

		if (CsvProfiler->IsCapturing())
		{
			UE_LOGFMT(LogAutomatedPerfTest, Warning, "Tried to begin csvprofile when csvprofiler was already capturing!");
		}
		else
		{
			FString CategoryNames;
			if (FParse::Value(*CommandString, TEXT("CATEGORY="), CategoryNames))
			{
				TArray<FString> Categories = TArray<FString>();
				CategoryNames.ParseIntoArray(Categories, TEXT(","));
				for (FString Category : Categories)
				{
					CsvProfiler->EnableCategoryByString(Category);
				}
			}

			int32 CaptureFrames;
			CaptureFrames = FParse::Value(*CommandString, TEXT("FRAMES="), CaptureFrames) ? CaptureFrames : -1;
			FString DestinationFolder;
			DestinationFolder = FParse::Value(*CommandString, TEXT("FOLDER="), DestinationFolder) ? DestinationFolder : FString();
			FString FileName = FString();
			if (FParse::Value(*CommandString, TEXT("FILENAME="), FileName))
			{
				// Set metadata for the scenario name which should be passed through with {Location}
				CsvProfiler->SetMetadata(TEXT("ProfileGoScenarioName"), *FileName);

				FileName = FileName.Replace(TEXT(":"), TEXT("-"));
				FileName = FPaths::MakeValidFileName(FileName);
				if (!FileName.EndsWith(TEXT(".csv")))
				{
					FileName.Append(TEXT(".csv"));
				}
			}

			if (UAutomatedPerfTestControllerBase* const Controller = ProfileGoSubsystem->GetTestController())
			{
				FileName = Controller ? Controller->GetTestID() + FileName : FileName;
				Controller->TryStartCSVProfiler(FileName, DestinationFolder, CaptureFrames);
			}
			else
			{
				FCsvProfiler::Get()->BeginCapture(CaptureFrames, DestinationFolder, FileName);
			}

			// Queue a command to emit the csv event on the next frame since the CsvProfiler will not begin capture until then
			FProfileGoCommandAPT BeginCsvEventCommand;
			BeginCsvEventCommand.Wait = -1.0f * FMath::Min<float>(TimeInState, 0.0f);
			TimeInState = FMath::Max<float>(TimeInState, 0.0f);
			BeginCsvEventCommand.Command = TEXT("csvevent begincsv");
			CurrentRequest->PendingCommands.EmplaceAt(1, BeginCsvEventCommand);
		}

		return true;
#else
		return false;
#endif //CSV_PROFILER
	}));

	RegisterCommandDelegate(TEXT("csvevent"),
	CommandHandlerDelegate::CreateLambda([this](FString& CommandString, const FProfileGoCommandAPT&)
	{
#if CSV_PROFILER
		FString CsvEventName = CommandString.RightChop(CommandString.Find(TEXT(" "), ESearchCase::IgnoreCase) + 1);
		if (CsvEventName.Equals(TEXT("begincsv"), ESearchCase::IgnoreCase))
		{
			CSV_EVENT(AutomatedPerfTest, TEXT("BeginCsv: %s"), *CurrentRequest->CurrentResult.Scenario.Name);
		}

		return true;
#else
		return false;
#endif //CSV_PROFILER
	}));

	RegisterCommandDelegate(TEXT("end csvprofile"),
	CommandHandlerDelegate::CreateLambda([this](FString& CommandString, const FProfileGoCommandAPT&)
	{
#if CSV_PROFILER
		if (!FCsvProfiler::Get()->IsCapturing())
		{
			UE_LOGFMT(LogAutomatedPerfTest, Warning, "Tried to end csvprofile when csvprofiler was not capturing!");
		}
		else
		{
			CSV_EVENT(AutomatedPerfTest, TEXT("EndCsv: %s"), *CurrentRequest->CurrentResult.Scenario.Name);
			if (UAutomatedPerfTestControllerBase* const Controller = ProfileGoSubsystem->GetTestController())
			{
				Controller->TryStopCSVProfiler();
			}
			else
			{
				FCsvProfiler::Get()->EndCapture();
			}
		}

		return true;
#else
		return false;
#endif //CSV_PROFILER
	}));

	RegisterCommandDelegate(TEXT("echo "),
	CommandHandlerDelegate::CreateLambda([this](FString& CommandString, const FProfileGoCommandAPT&)
	{
		FString Text = CommandString.RightChop(5);
		ClientMessage(*Text);
		return true;
	}));

	RegisterCommandDelegate(TEXT("gameplayscreenshot"),
	CommandHandlerDelegate::CreateLambda([this](FString& CommandString, const FProfileGoCommandAPT& ProfileGoCommand)
	{
		FString ScenarioName = FPaths::MakeValidFileName(CurrentRequest->CurrentResult.Scenario.Name);
		ScenarioName = ScenarioName.Replace(TEXT(" "), TEXT("_"));
		FString ScreenshotCommand = FString::Printf(TEXT("TakeGameplayAutomationScreenshot %s"), *ScenarioName);
		ConsoleCommand(*ScreenshotCommand);
		return true;
	}));

	RegisterCommandDelegate(TEXT("WaitOnStreaming"),
	CommandHandlerDelegate::CreateLambda([this](FString& CommandString, const FProfileGoCommandAPT&)
	{
		WaitForStreamingCommandComplete();
		return true;
	}));

}

void UProfileGo::RegisterInternalGeneratedScenarioDelegates()
{
	RegisterGeneratedScenarioDelegate(TEXT("here:"),
	GeneratedScenarioHandlerDelegate::CreateLambda([this](FString& ScenarioCommand, FString&)
	{
		FString CommandName;
		FString ScenarioName;
		if (ScenarioCommand.Split(TEXT(":"), &CommandName, &ScenarioName))
		{
			CreateScenarioHere(*ScenarioName);
			return AddScenariosToCurrentRequest(*ScenarioName);
		}
		return false;
	}));

	RegisterGeneratedScenarioDelegate(TEXT("fixedgrid:"),
	GeneratedScenarioHandlerDelegate::CreateLambda([this](FString& ScenarioCommand, FString& InProfileArgs)
	{
	    FString ProfileName = ScenarioCommand;

		const FProfileGoCollectionAPT* Exists = GetCollectionTemplates().FindByPredicate([ScenarioCommand](const FProfileGoCollectionAPT& Col)
		{
			return Col.Name == ScenarioCommand;
		});

		if (Exists == nullptr)
		{
			FString CommandName;
			FString GridSteps;
			if (ScenarioCommand.Split(TEXT(":"), &CommandName, &GridSteps))
			{
				FVector GridOrigin(ForceInit);
				if (FString GridOriginString; TryParseValue(*InProfileArgs, TEXT("-grid.origin="), GridOriginString))
				{
					TArray<FString> OriginComponents;
					if (GridOriginString.ParseIntoArray(OriginComponents, TEXT("x")) >= 2)
					{
						float GridOriginX = FCString::Atof(*OriginComponents[0]);
						float GridOriginY = FCString::Atof(*OriginComponents[1]);
						float GridOriginZ = OriginComponents.Num() > 2 ? FCString::Atof(*OriginComponents[2]) : 0.0f;
						GridOrigin = FVector(GridOriginX, GridOriginY, GridOriginZ);
						ClientMessage(*FString::Printf(TEXT("Setting grid origin to absolute world coordinates: %s"), *GridOrigin.ToCompactString()));
					}
				}
				else
				{
					ClientMessage(TEXT("Grid origin is at world zero coordinates"));
				}

				// Allow requesting a grid of a specific size by using XYZ box
				FBox NewBounds = FBox(ForceInit);
				if (FString GridBoxExtentString; TryParseValue(*InProfileArgs, TEXT("-grid.extent="), GridBoxExtentString))
				{
					TArray<FString> ExtentComponents;
					if (GridBoxExtentString.ParseIntoArray(ExtentComponents, TEXT("x")) >= 2)
					{
						double GridBoxExtentX = FCString::Atod(*ExtentComponents[0]);
						double GridBoxExtentY = FCString::Atod(*ExtentComponents[1]);
						double RetraceZExtent = ExtentComponents.Num() > 2 ? FCString::Atod(*ExtentComponents[2]) : 1000000.0;
						FVector GridBoxMin = FVector(-GridBoxExtentX, -GridBoxExtentY, -RetraceZExtent);
						FVector GridBoxMax = FVector(GridBoxExtentX, GridBoxExtentY, RetraceZExtent);

						GridBoxMin += GridOrigin;
						GridBoxMax += GridOrigin;
						NewBounds = FBox(GridBoxMin, GridBoxMax);
					}
				}

				if (!NewBounds.IsValid)
				{
					UE_LOGF(LogAutomatedPerfTest, Error, "No valid bounds specified and world doesn't have one either! Skipping scenario %ls", *ProfileName);
					return false;
				}

				if (!NewBounds.IsInside(GridOrigin))
				{
					UE_LOGF(LogAutomatedPerfTest, Error, "Grid origin (%ls) is not inside grid bounds (%ls)! Skipping scenario %ls", *GridOrigin.ToCompactString(), *NewBounds.ToString(), *ProfileName);
					return false;
				}

				const bool bIgnoreKillVolume = FParse::Param(*InProfileArgs, TEXT("ignorekillvolume"));
				const bool bIgnoreBlockingVolume = FParse::Param(*InProfileArgs, TEXT("ignoreblockingvolume"));

				if (TArray<FString> Components; GridSteps.ParseIntoArray(Components, TEXT("x")) >= 1)
				{
					const double GridStepX = FCString::Atod(*Components[0]);
					const double GridStepY = Components.Num() > 1 ? FCString::Atod(*Components[1]) : GridStepX;
					const int NumAngles = Components.Num() > 2 ? FCString::Atoi(*Components[2]) : 1;
					const float DegreeSteps = 360.f / NumAngles;
					const float WorldHeight = NewBounds.Max.Z;

					FProfileGoCollectionAPT NewCollection;
					NewCollection.Name = ProfileName;

					double StartPointX = FMath::GridSnap(NewBounds.Min.X - GridOrigin.X, GridStepX) + GridOrigin.X;
					double StartPointY = FMath::GridSnap(NewBounds.Min.Y - GridOrigin.Y, GridStepY) + GridOrigin.Y;

					// Fix GridSnap snaps to outside the box if lower bound is < 0
					if (StartPointX < NewBounds.Min.X)
					{
						StartPointX += GridStepX;
					}
					if (StartPointY < NewBounds.Min.Y)
					{
						StartPointY += GridStepY;
					}

					for (double PosX = StartPointX; PosX < NewBounds.Max.X; PosX += GridStepX)
					{
						const int64 i = FMath::FloorToInt((PosX - GridOrigin.X + (GridStepX / 2)) / GridStepX);
						for (double PosY = StartPointY; PosY < NewBounds.Max.Y; PosY += GridStepY)
						{
							const int64 j = FMath::FloorToInt((PosY - GridOrigin.Y + (GridStepY / 2)) / GridStepY);

							FVector TraceVector(PosX, PosY, WorldHeight);
							if (!FindBestStandingPoint(
								GetWorld(),
								TraceVector,
								TraceVector,
								bIgnoreKillVolume,
								bIgnoreBlockingVolume))
							{
								continue;
							}

							for (int SavedRotations = 0; SavedRotations < NumAngles; SavedRotations++)
							{
								FString ScenarioName = FString::Printf(TEXT("fixedgrid:%.0fx%.0f"), GridStepX, GridStepY);
								if (NumAngles > 1)
								{
									ScenarioName.Appendf(TEXT("x%i"), NumAngles);
								}
								ScenarioName.Appendf(TEXT("-%llix%lli"), i, j);
								if (NumAngles > 1)
								{
									ScenarioName.Appendf(TEXT("x%i"), SavedRotations + 1);
								}

								FProfileGoScenarioAPT NewProfileScenario;
								NewProfileScenario.Position = TraceVector;
								NewProfileScenario.Orientation = FRotator(0, DegreeSteps * SavedRotations, 0);
								NewProfileScenario.Name = ScenarioName;
								NewProfileScenario.AutoGenerated = true;

								// Add to the collection
								NewCollection.Scenarios.Add(ScenarioName);

								// Add all grid point scenarios regardless of if we need them in this collection.
								AddNewProfileGoScenario(NewProfileScenario);
							}
						}
					}

					AddNewProfileGoCollection(NewCollection);
				}
				else
				{
					UE_LOGF(LogAutomatedPerfTest, Error, "Grid steps are not specified! Skipping scenario %ls", *ProfileName);
					return false;
				}
			}
		}

		return AddScenariosToCurrentRequest(*ProfileName);
	}));

	RegisterGeneratedScenarioDelegate(TEXT("wp:"),
	GeneratedScenarioHandlerDelegate::CreateLambda([this](FString& ScenarioCommand, FString& InProfileArgs)
	{
	    FString ProfileName = ScenarioCommand;
		const FProfileGoCollectionAPT* Exists = GetCollectionTemplates().FindByPredicate([ProfileName](const FProfileGoCollectionAPT& Col)
		{
			return Col.Name == ProfileName;
		});

		const FProfileGoGeneratedScenarioAPT* GeneratedScenarioExists = ProfileGoGeneratedScenarios.FindByPredicate(
		[ProfileName](const FProfileGoGeneratedScenarioAPT& GeneratedScenario)
		{
			return GeneratedScenario.Name.Equals(ProfileName, ESearchCase::IgnoreCase);
		});

		if (Exists == nullptr)
		{
			UWorldPartition* WorldPartition = GetWorld()->GetWorldPartition();
			if (!WorldPartition)
			{
				return false;
			}

			const UWorldPartitionRuntimeSpatialHash* RuntimeHash = Cast<UWorldPartitionRuntimeSpatialHash>(WorldPartition->RuntimeHash);
			if (!RuntimeHash)
			{
				return false;
			}

			FString CommandName;
			FString CommandLineStr;
			ScenarioCommand.Split(TEXT(":"), &CommandName, &CommandLineStr);

			TArray<FString> Components;
			CommandLineStr.ParseIntoArray(Components, TEXT("x"));

			double StreamingRadius = 1;
			int RotationSteps = 1;
			static const FName GridName = TEXT("MainGrid");

			if (Components.Num() > 0)
			{
				RotationSteps = FMath::Max(1, FCString::Atoi(*Components[0]));
			}

			if (Components.Num() > 1)
			{
				StreamingRadius = FCString::Atof(*Components[1]);
				if (FMath::IsNearlyZero(StreamingRadius))
				{
					const FWorldPartitionStreamingSource* Source = WorldPartition->GetStreamingSources().
						FindByPredicate([](const FWorldPartitionStreamingSource& Source) { return Source.TargetGrids.Contains(GridName); });
					if (!Source)
					{
						ClientMessage(TEXT("Streaming source couldn't be found"));
						return false;
					}

					if (Source->Shapes.Num() != 1)
					{
						ClientMessage(TEXT("Streaming source is expected to have only 1 shape assigned"));
						return false;
					}

					StreamingRadius = Source->Shapes[0].Radius;
				}
			}

			int32 CellSize = 1;
			int32 StepDistance = 1;
			FBox IntBounds;
			FBox ContentBounds;
			TArray<TPair<FIntPoint, FVector>> ValidCells;
			RuntimeHash->ForEachStreamingGrid([&](const FSpatialHashStreamingGrid& Grid)
				{
					if (Grid.GridName != GridName)
					{
						return;
					}

					CellSize = 2 * Grid.CellSize; // It's actually cell Extent
					StepDistance = FMath::CeilToInt(StreamingRadius * UE_DOUBLE_INV_SQRT_2 / CellSize + 0.5);

					Grid.ForEachRuntimeCell([&](const UWorldPartitionRuntimeCell* Cell)
						{
							const UWorldPartitionRuntimeCellDataSpatialHash* CellData = Cast<UWorldPartitionRuntimeCellDataSpatialHash>(Cell->RuntimeCellData);

							if (!CellData)
							{
								return true;
							}

							if (CellData->HierarchicalLevel != 0 || !CellData->ContentBounds.IsValid)
							{
								return true;
							}

							const FVector OriginalPosition = CellData->Position;
							FVector TraceVector = CellData->Position;
							TraceVector.Z = CellData->ContentBounds.Max.Z;

							if (!FindBestStandingPoint(GetWorld(), TraceVector, TraceVector))
							{
								return true;
							}

							const FIntVector Index = FIntVector(OriginalPosition / CellSize);
							IntBounds += FVector(Index.X, Index.Y, 0);
							ContentBounds += CellData->ContentBounds;
							ValidCells.Emplace(FIntPoint(Index.X, Index.Y), TraceVector);

							return true;
						});
				});

			ValidCells.Sort([](const TPair<FIntPoint, FVector>& Lhs, const TPair<FIntPoint, FVector>& Rhs)
				{
					return std::tie(Lhs.Key.X, Lhs.Key.Y) < std::tie(Rhs.Key.X, Rhs.Key.Y);
				});

			if (ValidCells.IsEmpty())
			{
				return false;
			}

			double MaxZHeight = ContentBounds.Max.Z;

			TArray<FString> Scenarios;
			for (int32 X = IntBounds.Min.X; X <= IntBounds.Max.X; X += StepDistance)
			{
				for (int32 Y = IntBounds.Min.Y; Y <= IntBounds.Max.Y; Y += StepDistance)
				{
					FVector Position(X * CellSize, Y * CellSize, MaxZHeight);

					if (TPair<FIntPoint, FVector>* ValidCell = ValidCells.FindByPredicate([X, Y](const TPair<FIntPoint, FVector>& Pair) { return Pair.Key == FIntPoint(X, Y); }))
					{
						Position.Z = ValidCell->Value.Z;
					}

					for (int SavedRotations = 0; SavedRotations < RotationSteps; SavedRotations++)
					{
						const FString ScenarioName = FString::Printf(
							TEXT("%s@%ix%ix%i"),
							*ProfileName, X, Y, SavedRotations);

						FProfileGoScenarioAPT NewProfileScenario;
						NewProfileScenario.Position = Position;
						NewProfileScenario.Orientation = FRotator(0, 360 / RotationSteps * SavedRotations, 0);
						NewProfileScenario.Name = ScenarioName;
						NewProfileScenario.AutoGenerated = true;

						if (GeneratedScenarioExists)
						{
							NewProfileScenario.OnBegin = GeneratedScenarioExists->OnBegin;
							NewProfileScenario.OverrideCommands = GeneratedScenarioExists->OverrideCommands;
							NewProfileScenario.OnEnd = GeneratedScenarioExists->OnEnd;
							NewProfileScenario.SkipRunCommands = GeneratedScenarioExists->SkipRunCommands;
						}

						Scenarios.Emplace(ScenarioName);

						AddNewProfileGoScenario(NewProfileScenario);
					}
				}
			}

			FProfileGoCollectionAPT NewCollection;
			NewCollection.Name = ProfileName;
			NewCollection.Scenarios = Scenarios;

			AddNewProfileGoCollection(NewCollection);
		}

		return AddScenariosToCurrentRequest(*ProfileName);
	}));

	auto LocLambda = [this](FString& ScenarioCommand, FString& InProfileArgs)
	{
		FString ProfileName = ScenarioCommand;
		FString FullName = FString::Printf(TEXT("%s %s"), *ScenarioCommand, *InProfileArgs);
		FString ScenarioName = ScenarioCommand;

		const FProfileGoCollectionAPT* Exists = GetCollectionTemplates().FindByPredicate([FullName](const FProfileGoCollectionAPT& Col)
		{
			return Col.Name == FullName;
		});

		const FProfileGoGeneratedScenarioAPT* GeneratedScenarioExists = ProfileGoGeneratedScenarios.FindByPredicate(
		[ProfileName](const FProfileGoGeneratedScenarioAPT& GeneratedScenario)
		{
			return GeneratedScenario.Name.Equals(ProfileName, ESearchCase::IgnoreCase);
		});

		if (Exists)
		{
			return AddScenariosToCurrentRequest(*FullName);
		}
		else
		{
			FString CommandName;
			FString Coords;

			// TODO this needs to also parse the BugItGo commandname
			if (FullName.Split(TEXT(":"), &CommandName, &Coords))
			{
				// remove brackets
				Coords.ReplaceInline(TEXT("("), TEXT(""));
				Coords.ReplaceInline(TEXT(")"), TEXT(""));
				TArray<FString> Components;
				FString UnderscoreDelimiter = TEXT("_");
				if (Coords.ParseIntoArrayWS(Components, *UnderscoreDelimiter, true) >= 3)
				{
					FVector Location;
					Location.X = FCString::Atof(*Components[0]);
					Location.Y = FCString::Atof(*Components[1]);
					Location.Z = FCString::Atof(*Components[2]);

					FRotator Rotation(ForceInitToZero);

					if (Components.Num() >= 6)
					{
						Rotation.Pitch = FCString::Atof(*Components[3]);
						Rotation.Yaw = FCString::Atof(*Components[4]);
						Rotation.Roll = FCString::Atof(*Components[5]);
					}

					FProfileGoScenarioAPT NewProfileScenario;
					NewProfileScenario.Position = Location;
					NewProfileScenario.Orientation = Rotation.ContainsNaN() ? FRotator(ForceInitToZero) : Rotation;
					NewProfileScenario.Name = ScenarioName;
					NewProfileScenario.AutoGenerated = true;

					if (GeneratedScenarioExists)
					{
						NewProfileScenario.OnBegin = GeneratedScenarioExists->OnBegin;
						NewProfileScenario.OverrideCommands = GeneratedScenarioExists->OverrideCommands;
						NewProfileScenario.OnEnd = GeneratedScenarioExists->OnEnd;
						NewProfileScenario.SkipRunCommands = GeneratedScenarioExists->SkipRunCommands;
					}

					AddNewProfileGoScenario(NewProfileScenario);
				}
			}
		}

		return AddScenariosToCurrentRequest(*ScenarioName);
	};

	RegisterGeneratedScenarioDelegate(TEXT("loc:"), GeneratedScenarioHandlerDelegate::CreateLambda(LocLambda));
	RegisterGeneratedScenarioDelegate(TEXT("BugItGo:"), GeneratedScenarioHandlerDelegate::CreateLambda(LocLambda));
}

APlayerController* UProfileGo::GetPlayerController() const
{
	if (ULocalPlayer* Player = GEngine->GetLocalPlayerFromControllerId(GetWorld(), 0))
	{
		FLocalPlayerContext Context(Player);
		return Context.GetPlayerController();
	}

	return nullptr;
}

void UProfileGo::SetState(EProfileGoStateAPT InState)
{
	if (CurrentState != EProfileGoStateAPT::None)
	{
		TRACE_END_REGION(*WriteToString<256>(TEXT("ProfileGo - "), GetStateName(CurrentState)));
	}

	if (InState != EProfileGoStateAPT::None)
	{
		TRACE_BEGIN_REGION(*WriteToString<256>(TEXT("ProfileGo - "), GetStateName(InState)));
	}
	CurrentState = InState;
	TimeInState = 0.0f;
	bFirstTickInState = true;
	ClientMessage(*FString::Printf(TEXT("Changed to state %s"), *GetStateName(InState)));
}

FString UProfileGo::GetStateName(EProfileGoStateAPT InState) const
{
	switch (InState)
	{
	case EProfileGoStateAPT::None: return TEXT("EProfileGoStateAPT::None");
	case EProfileGoStateAPT::Completed: return TEXT("EProfileGoStateAPT::Completed");
	case EProfileGoStateAPT::CompletedScenario: return TEXT("EProfileGoStateAPT::CompletedScenario");
	case EProfileGoStateAPT::RunningCommands: return TEXT("EProfileGoStateAPT::RunningCommands");
	case EProfileGoStateAPT::StartingRequest: return TEXT("EProfileGoStateAPT::StartingRequest");
	case EProfileGoStateAPT::SettlingLocation: return TEXT("EProfileGoStateAPT::SettlingLocation");
	case EProfileGoStateAPT::Summary: return TEXT("EProfileGoStateAPT::Summary");
	}

	return TEXT("Invalid");
}

bool UProfileGo::FindBestStandingPoint(UWorld* InWorld, FVector StartXyHalfHeightZ, FVector& OutPos, bool bIgnoreKillVolume /*= false*/, bool bIgnoreBlockingVolume /*= false*/)
{
	const float SafetySpace = 100.f;

	FVector TraceStart(StartXyHalfHeightZ.X, StartXyHalfHeightZ.Y, StartXyHalfHeightZ.Z);
	FVector TraceEnd(StartXyHalfHeightZ.X, StartXyHalfHeightZ.Y, -StartXyHalfHeightZ.Z);
	OutPos = TraceEnd;

	if (!InWorld)
	{
		UE_LOGFMT(LogAutomatedPerfTest, Log, "InWorld is nullptr! Cannot trace for best standing point.");
		return false;
	}

	UE_LOGF(LogAutomatedPerfTest, Log, "Line casting from %ls to %ls", *TraceStart.ToString(), *TraceEnd.ToString());
	bool bTracedHit = false;
	FHitResult WaterHit;
	// TODO: Allow customization of trace channels
	bTracedHit |= InWorld->LineTraceSingleByChannel(WaterHit, TraceStart, TraceEnd, ECC_GameTraceChannel15);
	FHitResult WorldHit;
	bTracedHit |= InWorld->LineTraceSingleByChannel(WorldHit, TraceStart, TraceEnd, ECC_GameTraceChannel11);

	if (bTracedHit)
	{
		// find the highest blocking trace hit
		if (WaterHit.bBlockingHit && WaterHit.ImpactPoint.Z > OutPos.Z)
		{
			OutPos.Z = WaterHit.ImpactPoint.Z;
			UE_LOGF(LogAutomatedPerfTest, Log, "Hit water object [%ls]", *WaterHit.GetActor()->GetName());
		}
		if (WorldHit.bBlockingHit && WorldHit.ImpactPoint.Z > OutPos.Z)
		{
			OutPos.Z = WorldHit.ImpactPoint.Z;
			UE_LOGF(LogAutomatedPerfTest, Log, "Hit world object [%ls]", *WorldHit.GetActor()->GetName());
		}
		OutPos.Z += SafetySpace; // Set the location a bit higher so that we don't try to teleport into the world
	}
	else
	{
		UE_LOGF(LogAutomatedPerfTest, Log, "Rejecting location (%.02f,%.02f) due to failed trace. No blocking collision between %.02f and %.02f Z.",
			TraceStart.X, TraceStart.Y, TraceStart.Z, TraceEnd.Z);
		TArray<FHitResult> HitResults;
		InWorld->LineTraceMultiByObjectType(HitResults, TraceStart, TraceEnd, FCollisionObjectQueryParams(FCollisionObjectQueryParams::InitType::AllObjects));
		for (FHitResult HitResult : HitResults)
		{
			UE_LOGF(LogAutomatedPerfTest, Log, "Unhit object: %ls at %ls",
				*HitResult.GetActor()->GetName(), *HitResult.GetActor()->GetActorLocation().ToCompactString());
		}
		return false;
	}

	if (!bIgnoreKillVolume)
	{
		for (TObjectIterator<AKillZVolume> It; It; ++It)
		{
			if (It->EncompassesPoint(OutPos, SafetySpace))
			{
				UE_LOGF(LogAutomatedPerfTest, Log, "Rejecting location (%.02f,%.02f,%.02f) due to encompassing KillZVolume %ls", OutPos.X, OutPos.Y, OutPos.Z, *It->GetFullName());
				return false;
			}
		}
	}

	if (!bIgnoreBlockingVolume)
	{
		for (TObjectIterator<ABlockingVolume> It; It; ++It)
		{
			if (It->EncompassesPoint(OutPos, SafetySpace))
			{
				UE_LOGF(LogAutomatedPerfTest, Log, "Rejecting location (%.02f,%.02f,%.02f) due to encompassing BlockingVolume %ls", OutPos.X, OutPos.Y, OutPos.Z, *It->GetFullName());
				return false;
			}
		}
	}

	return true;
}

TArray<FProfileGoCommandAPT> UProfileGo::CreateCommandList(const FString& InCommandString)
{
	TArray<FProfileGoCommandAPT> OutCommands;

	TArray<FString> CommandStrings;
	InCommandString.ParseIntoArray(CommandStrings, TEXT(","));

	for (const FString& CommandString : CommandStrings)
	{
		UE_LOGF(LogAutomatedPerfTest, Log, "Looking for group %ls", *CommandString);

		bool bFound = false;

		// Check if this is the name of a group. If so, add all commands from that group in order.
		for (const FProfileGoCommandAPT& ProfileGoCommand : GetCommandTemplates())
		{
			if (ProfileGoCommand.Group.Equals(CommandString, ESearchCase::IgnoreCase))
			{
				bFound = true;
				UE_LOGF(LogAutomatedPerfTest, Log, "Added command %ls with wait %ls from group %ls", *ProfileGoCommand.Command, *FString::SanitizeFloat(ProfileGoCommand.Wait), *ProfileGoCommand.Group);
				OutCommands.Add(ProfileGoCommand);
			}
		}

		// If the command is not a group name, assume it is a console command and parse for profilego params like -pgc.wait=
		if (!bFound)
		{
			FProfileGoCommandAPT NewCommand;
			NewCommand.Command = CommandString;
			FString DebugInfo = FString::Printf(TEXT("Added console command '%s'"), *NewCommand.Command);

			FString FoundLog;
			if (FParse::Value(*CommandString, TEXT("-PGC.LOG="), FoundLog))
			{
				NewCommand.Log = FoundLog;
				DebugInfo.Appendf(TEXT(" redirected to file named %s"), *FoundLog);
			}

			if (FParse::Param(*CommandString, TEXT("-PGC.SKIPGAMELOG")))
			{
				NewCommand.CopyOutputToGameLog = false;
				DebugInfo.Appendf(TEXT(" that skips output to game log"));
			}

			bool bWaitForCallback = FParse::Param(*CommandString, TEXT("-PGC.WAITFORCALLBACK"));
			if (bWaitForCallback)
			{
				NewCommand.WaitForCallback = true;
				DebugInfo.Appendf(TEXT(" that waits for a callback"));
			}

			float FoundWait = 0.f;
			if (FParse::Value(*CommandString, TEXT("-PGC.WAIT="), FoundWait))
			{
				NewCommand.Wait = FoundWait;
				DebugInfo.Appendf(TEXT(" with a %s wait of %s seconds")
					, bWaitForCallback ? TEXT("max") : TEXT("min")
					, *FString::SanitizeFloat(FoundWait));
			}

			OutCommands.Add(NewCommand);
			UE_LOGF(LogAutomatedPerfTest, Log, "%ls", *DebugInfo);
		}
	}

	return OutCommands;
}

bool UProfileGo::TryExecuteNextPendingCommand()
{
	if (bWaitingOnCommandComplete)
	{
		return true;
	}

	if (CurrentRequest->PendingCommands.Num())
	{
		// execute the next command in the list
		const FProfileGoCommandAPT ProfileGoCommand = CurrentRequest->PendingCommands[0];
		ExecuteProfileGoCommand(ProfileGoCommand);
		CurrentRequest->PendingCommands.RemoveAt(0);
		return true;
	}
	else
	{
		StopCommandLog();
	}

	return false;
}

FProfileGoScenarioAPT UProfileGo::CreateScenarioHere(const TCHAR* ScenarioName)
{
	FProfileGoScenarioAPT NewScenario;
	FString LocationName = ScenarioName;
	if (LocationName.Len() == 0)
	{
		static int32 Count = 0;
		LocationName = FString::Printf(TEXT("Unnamed%d"), ++Count);
	}
	NewScenario.Name = LocationName;
	NewScenario.AutoGenerated = false;

	if (APlayerController* Controller = GetPlayerController())
	{
		FVector ViewLocation(ForceInitToZero);
		FRotator ViewRotation(ForceInitToZero);
		Controller->GetPlayerViewPoint(ViewLocation, ViewRotation);

		if (Controller->GetPawn() != NULL)
		{
			ViewLocation = Controller->GetPawn()->GetActorLocation();
		}

		NewScenario.Orientation = ViewRotation;
		NewScenario.Position = ViewLocation;
	}

	AddNewProfileGoScenario(NewScenario);

	return NewScenario;
}


bool UProfileGo::CollectionExists(const FString& Name)
{
	return GetCollectionTemplates().ContainsByPredicate([Name](FProfileGoCollectionAPT Collection)
		{
			return Collection.Name.Equals(Name);
		});
}

bool UProfileGo::ScenarioExists(const FString& Name)
{
	return GetScenarioTemplates().ContainsByPredicate([Name](FProfileGoScenarioAPT Scenario)
		{
			return Scenario.Name.Equals(Name);
		});
}

void UProfileGo::UpdateProjectSettings()
{
	if (UAutomatedProfileGoTestProjectSettings* Settings = GetMutableDefault<UAutomatedProfileGoTestProjectSettings>())
	{
		const bool bHasLoadedConfig = GetScenarioTemplates().Num() > 0 && GetCollectionTemplates().Num() > 0;
		if (bHasLoadedConfig)
		{
			Settings->Scenarios = GetScenarioTemplates();
			Settings->Collections = GetCollectionTemplates();
			Settings->Commands = GetCommandTemplates();
			Settings->SaveConfig();
		}
	}
}

void UProfileGo::AddScenariosInLevel(UWorld* World)
{
#if WITH_EDITOR
    // Look for actors of known type in level and use the corresponding transform
	// as "Scenario". The actor also has metadata for the "Collection" the scenario
	// should belong to i.e. "Group". Once discovered, add them to the list of available
	// scenarios. 
	TArray<AActor*> FoundCameras;
	UGameplayStatics::GetAllActorsOfClass(World, AAutomatedPerfTestStaticCamera::StaticClass(), FoundCameras);

	for (AActor* Actor : FoundCameras)
	{
		if (AAutomatedPerfTestStaticCamera* ActorAPT = Cast<AAutomatedPerfTestStaticCamera>(Actor))
		{
			FVector Translation = ActorAPT->GetActorLocation();
			FRotator Rotator = ActorAPT->GetActorRotation();
			FString CollectionName = ActorAPT->CollectionName.IsEmpty() ? TEXT("Default") : ActorAPT->CollectionName;

			FString ScenarioName = ActorAPT->GetActorLabel();

			// The only thing identifying a scenario is 
			// its name at the moment. 
			if (!ScenarioExists(ScenarioName))
			{
				if (ScenarioName.Len() == 0)
				{
					const int32 Count = GetScenarioTemplates().Num();
					ScenarioName = FString::Printf(TEXT("Unnamed%d"), Count + 1);
				}

				FProfileGoScenarioAPT Scenario = {};
				Scenario.Name = ScenarioName;
				Scenario.Orientation = Rotator;
				Scenario.Position = Translation;
				Scenario.AutoGenerated = false;
				AddNewProfileGoScenario(Scenario);
			}
			else
			{
				FProfileGoScenarioAPT* Scenario = GetScenarioTemplates().FindByPredicate([ScenarioName](FProfileGoScenarioAPT Scenario) { return Scenario.Name.Equals(ScenarioName); });
				Scenario->Orientation = Rotator;
				Scenario->Position = Translation;
			}

			if (!CollectionExists(CollectionName))
			{
				FProfileGoCollectionAPT Collection = {};
				Collection.Name = CollectionName;
				AddNewProfileGoCollection(Collection);
			}

			FProfileGoCollectionAPT* Collection = GetCollectionTemplates().FindByPredicate(
			[CollectionName](FProfileGoCollectionAPT Collection)
			{
				return Collection.Name.Equals(CollectionName);
			});

			if (Collection != nullptr)
			{
				// Add scenario to corresponding collection if it does not exist.
				const bool bContainsScenario = Collection->Scenarios.ContainsByPredicate([ScenarioName](FString Scenario)
				{ 
					return Scenario.Equals(ScenarioName, ESearchCase::IgnoreCase); 
				});

				if (!bContainsScenario)
				{
					Collection->Scenarios.Add(ScenarioName);
				}
			}
		}
	}

	UpdateProjectSettings();
#endif
}

TArray<FProfileGoCommandAPT> FProfileGoGeneratedScenarioAPT::Generate() const
{
	return {};
}
