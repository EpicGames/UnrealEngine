// Copyright Epic Games, Inc. All Rights Reserved.

#include "ProfileGo/ProfileGoSubsystem.h"
#include "ProfileGo/ProfileGo.h"

#include "AutomatedPerfTestControllerBase.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ProfileGoSubsystem)

void UProfileGoSubsystem::Initialize(FSubsystemCollectionBase& Collection)
{
	Super::Initialize(Collection);
	ProfileGo = nullptr;
	TestController = nullptr;
	UProfileGo::GetCDO().UpdateProjectSettings();
}

void UProfileGoSubsystem::Deinitialize()
{
	Super::Deinitialize();
	ProfileGo = nullptr;
}

bool UProfileGoSubsystem::IsTickable() const
{
	return ProfileGo && ProfileGo->IsRunning();
}

void UProfileGoSubsystem::Tick(float DeltaTime)
{
	if (ProfileGo)
	{
		ProfileGo->Tick(DeltaTime);
	}
}

bool UProfileGoSubsystem::Run(TSubclassOf<UProfileGo> ProfileGoClass, const TCHAR* ProfileName, const TCHAR* ProfileArgs)
{
	if (ProfileGo && ProfileGo->IsRunning())
	{
		return false;
	}

	ProfileGo = NewObject<UProfileGo>(this, ProfileGoClass);
	return ProfileGo->Run(this, ProfileName, ProfileArgs);
}
