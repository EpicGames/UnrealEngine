// Copyright Epic Games, Inc. All Rights Reserved.

#include "StaticCameraTests/AutomatedPerfTestStaticCamera.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AutomatedPerfTestStaticCamera)

AAutomatedPerfTestStaticCamera::AAutomatedPerfTestStaticCamera(const FObjectInitializer& ObjectInitializer)
: Super(ObjectInitializer)
{
	// disable tick on these actors, as it's not needed
	AActor::SetActorTickEnabled(false);
}
