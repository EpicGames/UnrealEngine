// Copyright Epic Games, Inc. All Rights Reserved.

#include "StaticCameraTests/AutomatedPlacedStaticCameraPerfTest.h"
#include "Kismet/GameplayStatics.h"
#include "StaticCameraTests/AutomatedPerfTestStaticCamera.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AutomatedPlacedStaticCameraPerfTest)

TArray<ACameraActor*> UAutomatedPlacedStaticCameraPerfTest::GetMapCameraActors()
{
	TArray<AActor*> FoundCameras;
	TArray<ACameraActor*> OutCameras;
	UGameplayStatics::GetAllActorsOfClass(GetWorld(), AAutomatedPerfTestStaticCamera::StaticClass(), FoundCameras);

	for(AActor* Actor : FoundCameras)
	{
		if(ACameraActor* FoundCamera = Cast<ACameraActor>(Actor))
		{
			OutCameras.Add(FoundCamera);	
		}
	}

	return OutCameras;
}
