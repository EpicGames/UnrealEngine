// Copyright Epic Games, Inc. All Rights Reserved.

#include "StaticCameraTests/AutomatedStaticCameraPerfTestBase.h"
#include "TimerManager.h"
#include "Kismet/GameplayStatics.h"
#include "AutomatedPerfTesting.h"
#include "Camera/CameraActor.h"
#include "Logging/StructuredLog.h"
#include "Misc/CommandLine.h"
#include "ProfilingDebugging/CsvProfiler.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AutomatedStaticCameraPerfTestBase)

UAutomatedStaticCameraPerfTestProjectSettings::UAutomatedStaticCameraPerfTestProjectSettings(const FObjectInitializer& Initializer)
	: Super(Initializer)
	, bCaptureScreenshots(true)
	, WarmUpTime(5.0)
	, SoakTime(5.0)
	, CooldownTime(1.0)
{

}

bool UAutomatedStaticCameraPerfTestProjectSettings::GetMapFromAssetName(FString AssetName, FSoftObjectPath& OutSoftObjectPath) const
{
	for (FSoftObjectPath MapPath : MapsToTest)
	{
		if(MapPath.GetAssetName() == AssetName)
		{
			OutSoftObjectPath = MapPath;
			return true;
		}
	}
	return false;
}

void UAutomatedStaticCameraPerfTestBase::SetupTest()
{
	// load up into the map defined in project settings
	if(!CurrentMapPath.IsNull())
	{
		if(GetCurrentMap() == CurrentMapPath.GetAssetName())
		{
			Super::SetupTest();
			UE_LOGFMT(LogAutomatedPerfTest, Log, "UAutomatedStaticCameraPerfTestBase::SetupTest");
			// make sure the world exists, then create a sequence player
			if(UWorld* const World = GetWorld())
			{
				CamerasToTest = GetMapCameraActors();

				if(CamerasToTest.Num() <= 0)
				{
					UE_LOGF(LogAutomatedPerfTest, Warning, "No cameras found in the map %ls, skipping to next map", *CurrentMapPath.GetAssetName());
					NextMap();
					return;
				}

				UE_LOGF(LogAutomatedPerfTest, Log, "Found %d cameras to test in map %ls", CamerasToTest.Num(), *CurrentMapPath.GetAssetName());

				GetFirstPlayerController()->SetCinematicMode(true, true, true);

				// Setup profiling once we have loaded the map. 
				SetupProfiling();
				
				// delay for WarmUpDelay, and call RunTest
				FTimerHandle UnusedHandle;
				GetWorld()->GetTimerManager().SetTimer(UnusedHandle, this, &UAutomatedStaticCameraPerfTestBase::RunTest, 1.0, false, Settings->WarmUpTime);
			}
			// if we have an invalid world, we can't run the test, so we should bail out
			else
			{
				UE_LOGFMT(LogAutomatedPerfTest, Error, "Invalid World when starting UAutomatedStaticCameraPerfTest, exiting...");
				EndTestFailure();
			}
		}
		else
		{
			UE_LOGF(LogAutomatedPerfTest, Log, "Current Map Name %ls is not expected %ls, calling NextMap.", *GetCurrentMap(), *CurrentMapPath.GetAssetName())
			NextMap();
		}
	}
	else
	{
		UE_LOGFMT(LogAutomatedPerfTest, Log, "Current Map Path has not been set, calling NextMap to trigger.");
		NextMap();
	}
}

void UAutomatedStaticCameraPerfTestBase::RunTest()
{
	Super::RunTest();

	UE_LOGFMT(LogAutomatedPerfTest, Log, "UAutomatedStaticCameraPerfTestBase::RunTest");
	MarkProfilingStart(); // Mark the start region on the profiler once the test starts
	
	FTimerHandle UnusedHandle;
	GetWorld()->GetTimerManager().SetTimer(UnusedHandle, this, &UAutomatedStaticCameraPerfTestBase::SetUpNextCamera, 1.0, false, Settings->WarmUpTime);
}

FString UAutomatedStaticCameraPerfTestBase::GetPerfTestTypeID() const
{
	return TEXT("StaticCamera");
}

void UAutomatedStaticCameraPerfTestBase::GatherTestMetadata(TArray<TPair<FString, FString>>& OutMetadata) const
{
	Super::GatherTestMetadata(OutMetadata);

	OutMetadata.Emplace(TEXT("MapName"), CurrentMapName);
	OutMetadata.Emplace(TEXT("CameraName"), GetCurrentCameraRegionName());
}

void UAutomatedStaticCameraPerfTestBase::SetUpNextCamera()
{
	if(CamerasToTest.Num() <= 0)
	{
		UE_LOGFMT(LogAutomatedPerfTest, Log, "No more cameras left to test, moving to next map.");
		NextMap();
		return;
	}

	CurrentCamera = CamerasToTest.Pop();

	UE_LOGF(LogAutomatedPerfTest, Log, "Setting up %ls to test", *CurrentCamera->GetActorNameOrLabel());

	GetFirstPlayerController()->SetViewTarget(CurrentCamera);
	FVector ViewLocation = CurrentCamera->GetActorLocation();
	FRotator ViewRotation = CurrentCamera->GetActorRotation();

	FString GoString = FString::Printf(TEXT("BugItGo %f %f %f %f %f %f"), ViewLocation.X, ViewLocation.Y, ViewLocation.Z, ViewRotation.Pitch, ViewRotation.Yaw, ViewRotation.Roll);
	
	FTimerHandle UnusedHandle;
	GetWorld()->GetTimerManager().SetTimer(UnusedHandle, this, &UAutomatedStaticCameraPerfTestBase::EvaluateCamera, 1.0, false, Settings->WarmUpTime);
}

void UAutomatedStaticCameraPerfTestBase::EvaluateCamera()
{
	MarkCameraStart();

	FTimerHandle UnusedHandle;
	GetWorld()->GetTimerManager().SetTimer(UnusedHandle, this, &UAutomatedStaticCameraPerfTestBase::FinishCamera, 1.0, false, Settings->SoakTime);
}

void UAutomatedStaticCameraPerfTestBase::FinishCamera()
{
	MarkCameraEnd();
	
	if(Settings->bCaptureScreenshots)
	{
		FTimerHandle UnusedHandle;
		GetWorld()->GetTimerManager().SetTimer(UnusedHandle, this, &UAutomatedStaticCameraPerfTestBase::ScreenshotCamera, 1.0, false, Settings->CooldownTime);		
	}
	else
	{
		FTimerHandle UnusedHandle;
		GetWorld()->GetTimerManager().SetTimer(UnusedHandle, this, &UAutomatedStaticCameraPerfTestBase::SetUpNextCamera, 1.0, false, Settings->CooldownTime);
	}
}

void UAutomatedStaticCameraPerfTestBase::ScreenshotCamera()
{
	TakeScreenshot(GetCurrentCameraRegionName());

	FTimerHandle UnusedHandle;
	GetWorld()->GetTimerManager().SetTimer(UnusedHandle, this, &UAutomatedStaticCameraPerfTestBase::SetUpNextCamera, 1.0, false, Settings->CooldownTime);
}

void UAutomatedStaticCameraPerfTestBase::NextMap()
{
	UE_LOGFMT(LogAutomatedPerfTest, Log, "UAutomatedStaticCameraPerfTestBase::NextMap");

	if(MapsToTest.Num() > 0)
	{
		CurrentMapPath = MapsToTest.Pop();
		UE_LOGF(LogAutomatedPerfTest, Log, "Setting up test for Map %ls", *CurrentMapPath.GetAssetName())

		// no need to prepend this with a ? since OpenLevel handles that part for us
		FString OptionsString;
		if(!Settings->GameModeOverride.IsEmpty())
		{
			UE_LOGF(LogAutomatedPerfTest, Log, "Game Mode overridden to %ls", *Settings->GameModeOverride)
			OptionsString += "game=" + Settings->GameModeOverride;
		}
		
		UE_LOGF(LogAutomatedPerfTest, Log, "Opening map %ls%ls", *CurrentMapPath.GetAssetName(), *OptionsString);
		UGameplayStatics::OpenLevel(AutomatedPerfTest::FindCurrentWorld(), *CurrentMapPath.GetAssetName(), true, OptionsString);
	}
	else
	{
		UE_LOGFMT(LogAutomatedPerfTest, Log, "UAutomatedStaticCameraPerfTestBase::NextMap, all maps complete, exiting after delay.");
		TriggerExitAfterDelay();
	}
}

TArray<ACameraActor*> UAutomatedStaticCameraPerfTestBase::GetMapCameraActors()
{
	UE_LOGFMT(LogAutomatedPerfTest, Warning, "GetMapCameraActors called in base class UAutomatedStaticCameraPerfTestBase, please ensure you've overridden this function in the subclass, and you're not using the base class as your test controller");
	return TArray<ACameraActor*>();
}

ACameraActor* UAutomatedStaticCameraPerfTestBase::GetCurrentCamera() const
{
	return CurrentCamera;
}

FString UAutomatedStaticCameraPerfTestBase::GetCurrentCameraRegionName() const
{
	return GetCurrentCamera()->GetActorNameOrLabel();
}

void UAutomatedStaticCameraPerfTestBase::MarkCameraStart()
{
	// safety check on the current material
	if(CurrentCamera)
	{
		if(RequestsInsightsTrace())
		{
			TRACE_BEGIN_REGION(*GetCurrentCameraRegionName());
		}
#if CSV_PROFILER
		if(RequestsCSVProfiler())
		{
			FString CSVFilename = GetTestID() + "_" + GetCurrentCameraRegionName();
			TryStartCSVProfiler(CSVFilename);
		}
#endif
	}
}

void UAutomatedStaticCameraPerfTestBase::MarkCameraEnd()
{
	// safety check on the current material
	if(CurrentCamera)
	{
		if(RequestsInsightsTrace())
		{
			TRACE_END_REGION(*GetCurrentCameraRegionName());
		}
#if CSV_PROFILER
		if(RequestsCSVProfiler())
		{
			TryStopCSVProfiler();
		}
#endif
	}
}

void UAutomatedStaticCameraPerfTestBase::OnInit()
{
	Super::OnInit();
	
	UE_LOGFMT(LogAutomatedPerfTest, Log, "UAutomatedStaticCameraPerfTestBase::OnInit");

	Settings = GetDefault<UAutomatedStaticCameraPerfTestProjectSettings>();

	// if an explicit map/sequence name was set from commandline, use this to override the test
	if (FParse::Value(FCommandLine::Get(), TEXT("AutomatedPerfTest.StaticCameraPerfTest.MapName="), CurrentMapName))
	{
		for (FSoftObjectPath MapPath : Settings->MapsToTest)
		{
			if (MapPath.GetAssetName() == CurrentMapName)
			{
				MapsToTest.Add(MapPath);
			}
		}
		if(MapsToTest.IsEmpty())
		{
			UE_LOGF(LogAutomatedPerfTest, Error, "Couldn't find a map name matching %ls in Static Camera Maps to Test setting. Exiting.", *CurrentMapName)
			EndTestFailure();
			return;
		}
	}
	// otherwise, use all the maps defined in project settings
	else
	{
		UE_LOGFMT(LogAutomatedPerfTest, Log, "No map name specified, testing all maps.");
		for(FSoftObjectPath MapName : Settings->MapsToTest)
		{
			MapsToTest.Add(MapName);
		}
	}
	
	UE_LOGF(LogAutomatedPerfTest, Log, "Initialized with %d MapSequence combos", MapsToTest.Num());
	
	// early out if there aren't actually any maps set in project settings
	if(Settings->MapsToTest.IsEmpty())
	{
		UE_LOGFMT(LogAutomatedPerfTest, Error, "No maps defined in the project's Automated Perf Test | Static Camera settings. Exiting test early.");
		EndTestFailure();
		return;
	}
}

void UAutomatedStaticCameraPerfTestBase::UnbindAllDelegates()
{
	Super::UnbindAllDelegates();
	
	if (UWorld* const World = GetWorld())
	{
		World->GetTimerManager().ClearAllTimersForObject(this);
	}

	CurrentCamera = nullptr;
	CamerasToTest.Empty();
}

void UAutomatedStaticCameraPerfTestBase::TriggerExitAfterDelay()
{
	MarkProfilingEnd();
	TeardownProfiling();
	Super::TriggerExitAfterDelay();
}
