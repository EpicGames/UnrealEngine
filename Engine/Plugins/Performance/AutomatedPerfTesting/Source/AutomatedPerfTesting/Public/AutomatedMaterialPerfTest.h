// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "AutomatedPerfTestControllerBase.h"
#include "AutomatedPerfTestProjectSettings.h"
#include "UObject/Object.h"
#include "Templates/SubclassOf.h"
#include "Engine/EngineTypes.h"
#include "Engine/DeveloperSettings.h"
#include "Camera/CameraTypes.h"
#include "AutomatedMaterialPerfTest.generated.h"

#define UE_API AUTOMATEDPERFTESTING_API

class AStaticMeshActor;


UCLASS(MinimalAPI, BlueprintType, Config=Engine, DefaultConfig, DisplayName="Automated Performance Testing | Materials")
class UAutomatedMaterialPerfTestProjectSettings : public UDeveloperSettings
{
	GENERATED_BODY()

public:
	UE_API UAutomatedMaterialPerfTestProjectSettings(const FObjectInitializer&);

	/** Gets the settings container name for the settings, either Project or Editor */
	virtual FName GetContainerName() const override { return FName("Project"); }
	/** Gets the category for the settings, some high level grouping like, Editor, Engine, Game...etc. */
	virtual FName GetCategoryName() const override { return FName("Plugins"); }
	
	/*
	 * List of materials to load and review for the Material Performance Test
	 */
	UPROPERTY(Config, EditAnywhere, BlueprintReadWrite, Category="Material Perf Test | Parameters", meta=(AllowedClasses="/Script/Engine.MaterialInterface"))
	TArray<FSoftObjectPath> MaterialsToTest;

	/*
	 * If true, will capture a screenshot for each material tested after gathering data
	 */
	UPROPERTY(Config, EditAnywhere, BlueprintReadWrite, Category="Material Perf Test | Parameters")
	bool bCaptureScreenshots;

	/*
	 * For how long the material performance test should delay before beginning to gather data for a material, in seconds
     */
	UPROPERTY(Config, EditAnywhere, BlueprintReadWrite, Category="Material Perf Test | Parameters")
	float WarmUpTime;
	
	/*
	 * For how long the material performance test should gather data on each material, in seconds
	 */
	UPROPERTY(Config, EditAnywhere, BlueprintReadWrite, Category="Material Perf Test | Parameters")
	float SoakTime;

	/*
	 * For how long the material performance test should delay after ending evaluation before switching to the next material
     */
	UPROPERTY(Config, EditAnywhere, BlueprintReadWrite, Category="Material Perf Test | Parameters")
	float CooldownTime;

	/*
	 * The map in which the material test will take place. Useful if you need to set up things like RVT volumes that are required
	 * by your materials.
	 */
	UPROPERTY(Config, EditAnywhere, BlueprintReadWrite, Category="Material Perf Test | Scene", meta=(AllowedClasses="/Script/Engine.World"))
	FSoftObjectPath MaterialPerformanceTestMap;

	/*
	 * If set, will launch the material performance test map with this game mode alias (make sure you've set the game mode alias in
	 * the Maps and Modes settings of your project!)
	 */
	UPROPERTY(Config, EditAnywhere, BlueprintReadWrite, Category="Material Perf Test | Scene")
	FString GameModeOverride;
	
	/*
     * Which camera projection mode to use
     */
	UPROPERTY(Config, EditAnywhere, BlueprintReadWrite, Category="Material Perf Test | Camera")
	TEnumAsByte<ECameraProjectionMode::Type> CameraProjectionMode;
	
	/*
     * How far away from the camera should the material test plate be placed
     * (will also be used for Ortho Width if the camera projection mode is orthographic)
     */
	UPROPERTY(Config, EditAnywhere, BlueprintReadWrite, Category="Material Perf Test | Plate")
	float PlateDistanceFromCamera;

	/*
     * Which static mesh to use as the material plate
     */
	UPROPERTY(Config, EditAnywhere, BlueprintReadWrite, Category="Material Perf Test | Plate")
	FSoftObjectPath MaterialPlate;
};

/**
 * 
 */
UCLASS(MinimalAPI)
class UAutomatedMaterialPerfTest : public UAutomatedPerfTestControllerBase
{
	GENERATED_BODY()
	
public:
	// ~Begin UAutomatedPerfTestControllerBase Interface
	UE_API virtual void SetupTest() override;
	
	UFUNCTION()
	UE_API virtual void RunTest() override;

	UE_API virtual FString GetPerfTestTypeID() const override;

	UE_API virtual void GatherTestMetadata(TArray<TPair<FString, FString>>& OutMetadata) const override;
	// ~End UAutomatedPerfTestControllerBase Interface
	
	UFUNCTION()
	UE_API void SetUpNextMaterial();

	UFUNCTION()
	UE_API void EvaluateMaterial();
	
	UFUNCTION()
	UE_API void FinishMaterialEvaluation();
	
	UFUNCTION()
	UE_API void ScreenshotMaterial();

	UE_API FString GetCurrentMaterialRegionName() const;

	UE_API void MarkMaterialStart();
	UE_API void MarkMaterialEnd();
	
protected:
	UE_API virtual void OnInit() override;
	UE_API virtual void UnbindAllDelegates() override;
	UE_API virtual void TeardownTest(bool bExitAfterTeardown = true) override;

	UE_API void OpenMaterialPerformanceTestMap() const;

private:
	ACameraActor* Camera;
	AStaticMeshActor* MaterialPlate;
	UMaterialInterface* CurrentMaterial;
	int CurrentMaterialIndex;
	const UAutomatedMaterialPerfTestProjectSettings* Settings;
};

#undef UE_API
