// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "AutomatedPerfTestProjectSettings.h"
#include "GauntletTestController.h"
#include "Engine/World.h"
#include "Engine/Engine.h"

#include "AutomatedPerfTestControllerBase.generated.h"

#ifndef UE_API
#define UE_API AUTOMATEDPERFTESTING_API
#endif

class AGameModeBase;

namespace AutomatedPerfTest
{
	static UWorld* FindCurrentWorld()
	{
		UWorld* World = nullptr;
		for (const FWorldContext& WorldContext : GEngine->GetWorldContexts())
		{
			if (WorldContext.WorldType == EWorldType::Game)
			{
				World = WorldContext.World();
			}
#if WITH_EDITOR
			else if (GIsEditor && WorldContext.WorldType == EWorldType::PIE)
			{
				World = WorldContext.World();
				if (World)
				{
					return World;
				}
			}
#endif
		}

		return World;
	}

	// New function because there could be more than one Game world alive at any point with MultiWorld
	static bool IsWorldLoaded(const FString & WorldName)
	{
		for (const FWorldContext& WorldContext : GEngine->GetWorldContexts())
		{
			if (WorldContext.WorldType == EWorldType::Game)
			{
				UWorld* World = WorldContext.World();
				if (World && World->GetName().Compare(WorldName) == 0)
				{
					return true;
				}
			}
#if WITH_EDITOR
			else if (GIsEditor && WorldContext.WorldType == EWorldType::PIE)
			{
				UWorld* World = WorldContext.World();
				if (World && World->GetName().Compare(WorldName) == 0)
				{
					return true;
				}
			}
#endif
		}

		return false;
	}
}
/**
 * 
 */
UCLASS(MinimalAPI)
class UAutomatedPerfTestControllerBase : public UGauntletTestController
{
	GENERATED_BODY()
public:
	UE_API void OnPreWorldInitializeInternal(UWorld* World, const UWorld::InitializationValues IVS);
	UE_API virtual void OnPreWorldInitialize(UWorld* World);

	UFUNCTION()
	UE_API void OnWorldBeginPlay();

// Base functionality
public:
	UE_API UAutomatedPerfTestControllerBase(const FObjectInitializer& ObjectInitializer);

	UE_API FString GetTestName() const;
	UE_API FString GetTestID() const;

	// Unique identifier for the class type provided to output artifact generation.
	UE_API virtual FString GetPerfTestTypeID() const;

	UE_API FString GetOverallRegionName();
	UE_API FString GetTraceChannels();
	
	UE_API bool RequestsInsightsTrace() const;
	UE_API bool RequestsCSVProfiler() const;
	UE_API bool RequestsFPSChart() const;
	UE_API bool RequestsVideoCapture() const;
	UE_API bool RequestsLockedDynRes() const;

	UE_API bool TryStartInsightsTrace();
	UE_API bool TryStopInsightsTrace();

	UE_API bool TryStartCSVProfiler(FString CSVFileName, const FString& DestinationFolder = FString(), int32 Frames = -1);
	UE_API bool TryStopCSVProfiler();
	
	UE_API bool TryStartFPSChart();
	UE_API bool TryStopFPSChart();

	UE_API bool TryStartVideoCapture();
	UE_API bool TryFinalizingVideoCapture(const bool bStopAutoContinue = false);

	// Gather test metadata key-value pairs for profiling instruments.
	//
	// Override this method to add controller specific metadata (map names, material names, etc). Always call 
	// Super::GatherTestMetadata() to include base metadata.
	UE_API virtual void GatherTestMetadata(TArray<TPair<FString, FString>>& OutMetadata) const;

	UE_API virtual void SetupTest();
	UE_API virtual void RunTest();
	UE_API virtual void TeardownTest(bool bExitAfterTeardown = true);
	UE_API virtual void TriggerExitAfterDelay();
	UE_API virtual void Exit();

	UE_API AGameModeBase* GetGameMode() const;

	UE_API void TakeScreenshot(FString ScreenshotName);

	UE_API void ConsoleCommand(const TCHAR* Cmd);

protected:
	// ~Begin UGauntletTestController Interface
	UE_API virtual void OnInit() override;
	UE_API virtual void OnTick(float TimeDelta) override;
	UE_API virtual void BeginDestroy() override;
	// ~End UGauntletTestController Interface

	// Shut down and queue an exit with an exit code of 0 to indicate success.
	UFUNCTION()
	void EndTestSuccess();

	// Shut down and queue an exit with an exit code of 1 to indicate generic failure. A custom non-zero value may be
	// provided if needed, though any new systemic failures should be implemented as a new function with a custom 
	// reserved value.
	UFUNCTION()
	void EndTestFailure(const int32 ExitCode = 1);

	UFUNCTION()
	UE_API virtual void OnVideoRecordingFinalized(bool Succeeded, const FString& FilePath);
	
	UE_API virtual void UnbindAllDelegates();

	UE_API void SetupGameModeInstance();

	UE_API APlayerController* GetPlayerController();
	
	/* Profiling Functions */
	UE_API void SetupProfiling();
	UE_API void InitializeInsights();
	UE_API void ShutdownInsights();
	UE_API void TeardownProfiling();
	UE_API void MarkProfilingStart();
	UE_API void MarkProfilingEnd();
	/* End Profiling Functions*/

private:
	// Shut down and request the process exit with the provided error code. 
	UE_API void EndAutomatedPerfTest(const int32 ExitCode);
	
	FString TraceChannels;
	FString TestDatetime;
	FString TestID;
	bool bRequestsFPSChart;
	bool bRequestsInsightsTrace;
	bool bRequestsCSVProfiler;
	bool bRequestsVideoCapture;
	bool bRequestsLockedDynRes;
	uint64 InsightsRegionID;
	FString ArtifactOutputPath;

	FText VideoRecordingTitle;
	
	AGameModeBase* GameMode;

	// Event for the most recent call to TryStopCSVProfiler. Used to ensure the resulting csv file is flushed 
	// before exiting via EndAutomatedPerfTest().
	FGraphEventRef CsvProfileEndCaptureEvent;

	FDelegateHandle CsvProfilerDelegateHandle;
};

#undef UE_API
