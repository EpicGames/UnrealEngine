// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "AutomatedPerfTestInterface.h"
#include "GameFramework/GameModeBase.h"
#include "AutomatedPerfTestGameModeBase.generated.h"

/**
 * 
 */
UCLASS(Blueprintable)
class AUTOMATEDPERFTESTING_API AAutomatedPerfTestGameModeBase : public AGameModeBase, public IAutomatedPerfTestInterface
{
		GENERATED_BODY()
	
public:

//~ Begin IAutomatedPerfTestInterface	
	UFUNCTION(BlueprintCallable, BlueprintImplementableEvent, Category="Automated Perf Test")
	void SetupTest(); virtual void SetupTest_Implementation();

	UFUNCTION(BlueprintCallable, BlueprintImplementableEvent, Category="Automated Perf Test")
	void TeardownTest(); virtual void TeardownTest_Implementation();

	UFUNCTION(BlueprintCallable, BlueprintImplementableEvent, Category="Automated Perf Test")
	void RunTest(); virtual void RunTest_Implementation();

	UFUNCTION(BlueprintCallable, BlueprintImplementableEvent, Category="Automated Perf Test")
	void Exit(); virtual void Exit_Implementation();
//~ End IAutomatedPerfTestInterface

	
};
