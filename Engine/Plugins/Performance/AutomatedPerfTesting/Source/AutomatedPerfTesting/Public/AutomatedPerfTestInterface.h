// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "UObject/Interface.h"
#include "AutomatedPerfTestInterface.generated.h"

#define UE_API AUTOMATEDPERFTESTING_API

// This class does not need to be modified.
UINTERFACE(MinimalAPI, BlueprintType)
class UAutomatedPerfTestInterface : public UInterface
{
	GENERATED_BODY()
};

/**
 * 
 */
class IAutomatedPerfTestInterface
{
	GENERATED_BODY()

	// Add interface functions to this class. This is the class that will be inherited to implement this interface.
public:
	UFUNCTION(BlueprintCallable, BlueprintImplementableEvent, Category="Automated Perf Test")
	UE_API void SetupTest();

	UFUNCTION(BlueprintCallable, BlueprintImplementableEvent, Category="Automated Perf Test")
	UE_API void TeardownTest();

	UFUNCTION(BlueprintCallable, BlueprintImplementableEvent, Category="Automated Perf Test")
	UE_API void RunTest();

	UFUNCTION(BlueprintCallable, BlueprintImplementableEvent, Category="Automated Perf Test")
	UE_API void Exit();
};

#undef UE_API
