// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "UObject/Object.h"
#include "Templates/SubclassOf.h"
#include "Engine/EngineTypes.h"
#include "Engine/DeveloperSettings.h"
#include "AutomatedPerfTestProjectSettings.generated.h"

#define UE_API AUTOMATEDPERFTESTING_API


UCLASS(MinimalAPI, Config=Engine)
class UAutomatedPerfTestProjectSettings : public UDeveloperSettings
{
	GENERATED_BODY()

public:
	UE_API UAutomatedPerfTestProjectSettings(const FObjectInitializer&);

	/** Gets the settings container name for the settings, either Project or Editor */
	virtual FName GetContainerName() const override { return FName("Project"); }
	/** Gets the category for the settings, some high level grouping like, Editor, Engine, Game...etc. */
	virtual FName GetCategoryName() const override { return FName("Plugins"); }

	/* How many seconds to wait before transition from Teardown to Exiting the test */
	UPROPERTY(Config, EditAnywhere, BlueprintReadWrite, Category="Automated Performance Testing")
	float TeardownToExitDelay = 5.0;
};


#undef UE_API
