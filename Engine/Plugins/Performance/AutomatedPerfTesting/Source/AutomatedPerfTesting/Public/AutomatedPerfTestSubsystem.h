// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Subsystems/EngineSubsystem.h"
#include "AutomatedPerfTestSubsystem.generated.h"

/**
 * 
 */
UCLASS(MinimalAPI)
class UAutomatedPerfTestSubsystem : public UEngineSubsystem
{
	GENERATED_BODY()

	UFUNCTION(BlueprintCallable, Category="Automated Perf Testing")
	static FString GetTestID();
	
};
