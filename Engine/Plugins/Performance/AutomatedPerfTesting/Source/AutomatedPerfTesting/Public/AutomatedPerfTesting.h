// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Modules/ModuleManager.h"
#include "ProfilingDebugging/CsvProfiler.h"

DECLARE_LOG_CATEGORY_EXTERN(LogAutomatedPerfTest, Log, All)
CSV_DECLARE_CATEGORY_EXTERN(AutomatedPerfTest);

class FAutomatedPerfTestingModule : public IModuleInterface
{
public:

	/** IModuleInterface implementation */
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;

private:
};
