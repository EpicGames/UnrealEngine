// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Engine/TimerHandle.h"
#include "CoreGlobals.h"
#include "AutomatedPerfTestControllerBase.h"
#include "Subsystems/WorldSubsystem.h"
#include "Stats/Stats.h"
#include "ProfileGo/ProfileGo.h"

#include "AutomatedProfileGoTest.generated.h"

#ifndef UE_API
#define UE_API AUTOMATEDPERFTESTING_API
#endif

class UProfileGoSubsystem;
class FOutputDeviceFile;

/**
 * Automated Perf Profile Go Test.
 */
UCLASS(MinimalAPI)
class UAutomatedProfileGoTest : public UAutomatedPerfTestControllerBase
{
	GENERATED_BODY()

public:
	UE_API UAutomatedProfileGoTest(const FObjectInitializer& ObjectInitializer);

	UE_API virtual void OnInit() override;
	UE_API virtual FString GetPerfTestTypeID() const override;
	UE_API virtual void SetupTest() override;
	UE_API virtual void OnTick(float TimeDelta) override;

protected:
	UE_API virtual void OnRequestFailed();
	UE_API virtual void OnScenarioStarted(const FString& InName);
	UE_API virtual void OnScenarioEnded(const FString& InName);
	UE_API virtual void OnPassStarted();
	UE_API virtual void OnPassEnded();
	UE_API virtual void UnbindAllDelegates() override;
	UE_API virtual void CompleteTest(bool bSuccess);

	UE_API void		LoadFromJSON(const FString& Filename);
	UE_API void		SaveToJSON(const FString& Filename);
	UE_API FString	GetProfileGoArgsString();

	bool	bPlayerSetupFinished;
	bool	bIgnorePawn;
	bool	bBoundDelegates;
	bool	bRequestedProfileGo;
	bool	bStartedProfileGo;
	bool	bShouldTick;
	FString ProfileGoStatus;
	FString ProfileGoConfigFile;

	FString ProfileGoCommand;
	FString ProfileGoScenario;
	int32	NumRequiredLoops;
	int32	NumCompletedLoops;
};


UCLASS(MinimalAPI, BlueprintType, Config = Engine, DefaultConfig, DisplayName = "Automated Performance Testing | ProfileGo")
class UAutomatedProfileGoTestProjectSettings : public UDeveloperSettings
{
	GENERATED_BODY()

public:
	UE_API UAutomatedProfileGoTestProjectSettings(const FObjectInitializer&);

	/** Gets the settings container name for the settings, either Project or Editor */
	virtual FName GetContainerName() const override { return FName("Project"); }
	/** Gets the category for the settings, some high level grouping like, Editor, Engine, Game...etc. */
	virtual FName GetCategoryName() const override { return FName("Plugins"); }

	UPROPERTY(Config, EditAnywhere, BlueprintReadWrite, Category = "ProfileGo")
	TArray<FProfileGoScenarioAPT>	Scenarios;

	UPROPERTY(Config, EditAnywhere, BlueprintReadWrite, Category = "ProfileGo")
	TArray<FProfileGoCollectionAPT>	Collections;

	UPROPERTY(Config, EditAnywhere, BlueprintReadWrite, Category = "ProfileGo")
	TArray<FProfileGoCommandAPT>	Commands;

	UPROPERTY(Config, EditAnywhere, BlueprintReadWrite, Category = "ProfileGo")
	TArray<FProfileGoGeneratedScenarioAPT> GeneratedScenarios;
};

#undef UE_API