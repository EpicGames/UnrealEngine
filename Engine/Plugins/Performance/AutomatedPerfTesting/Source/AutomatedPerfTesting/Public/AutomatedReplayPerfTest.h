// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Object.h"
#include "Templates/SubclassOf.h"
#include "Engine/EngineTypes.h"
#include "Engine/DeveloperSettings.h"
#include "CoreMinimal.h"
#include "AutomatedPerfTestControllerBase.h"
#include "Net/Core/Connection/NetResult.h"
#include "Net/ReplayResult.h"

#include "AutomatedReplayPerfTest.generated.h"

#define UE_API AUTOMATEDPERFTESTING_API


UCLASS(MinimalAPI, BlueprintType, Config = Engine, DefaultConfig, DisplayName = "Automated Performance Testing | Replay")
class UAutomatedReplayPerfTestProjectSettings : public UDeveloperSettings
{
	GENERATED_BODY()

public:
	UE_API UAutomatedReplayPerfTestProjectSettings(const FObjectInitializer&);

	/** Gets the settings container name for the settings, either Project or Editor */
	virtual FName GetContainerName() const override { return FName("Project"); }
	/** Gets the category for the settings, some high level grouping like, Editor, Engine, Game...etc. */
	virtual FName GetCategoryName() const override { return FName("Plugins"); }

	/* Path to replay file to be used during Replay Perf Test */
	UPROPERTY(Config, EditAnywhere, BlueprintReadWrite, Category = "Replay Perf Test", meta = (FilePathFilter = "replay", RelativeToGameDir))
	TArray<FFilePath>				ReplaysToTest;

	/* Get replay path from given replay name */
	UFUNCTION(BlueprintCallable, Category = "Replay Perf Test")
	UE_API bool GetReplayPathFromName(FName TestName, FString& FoundReplay) const;
};

/**
 *
 */
UCLASS(MinimalAPI)
class UAutomatedReplayPerfTest : public UAutomatedPerfTestControllerBase
{
	GENERATED_BODY()

public:
	// ~Begin UAutomatedPerfTestControllerBase Interface
	UE_API virtual FString GetPerfTestTypeID() const override;
	UE_API virtual void SetupTest() override;

	UFUNCTION()
	UE_API virtual void RunTest() override;
	UE_API virtual void TeardownTest(bool bExitAfterTeardown = true) override;

protected:
	UE_API virtual void OnInit() override;
	UE_API virtual void UnbindAllDelegates() override;

	UE_API void OnReplayStarted(UWorld* World);
	UE_API void OnReplayComplete(UWorld* World);
	UE_API void OnReplayFailure(UWorld* World, const UE::Net::TNetResult<EReplayResult>& Error);

private:
	FString ReplayName;
	bool	bIsReplayTriggered	= false;
	bool	bIsTearingDown		= false;
};

#undef UE_API
