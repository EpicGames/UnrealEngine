// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Object.h"
#include "Templates/SubclassOf.h"
#include "Engine/EngineTypes.h"
#include "Engine/DeveloperSettings.h"
#include "CoreMinimal.h"
#include "AutomatedPerfTestControllerBase.h"
#include "Camera/CameraComponent.h"
#include "AutomatedSequencePerfTest.generated.h"

#define UE_API AUTOMATEDPERFTESTING_API


class ULevelSequence;
class ALevelSequenceActor;
class ULevelSequencePlayer;


USTRUCT(BlueprintType)
struct FAutomatedPerfTestMapSequenceCombo
{
	GENERATED_BODY()

	/**
	 * Use this name to directly reference this map/sequence combo via BuildGraph or UAT with -AutomatedPerfTest.SequenceTest.MapSequenceComboName
	 */
	UPROPERTY(Config, EditAnywhere, BlueprintReadWrite, Category="Sequence Perf Test", meta=(AllowedClasses="/Script/Engine.World"))
	FName ComboName;

	UPROPERTY(Config, EditAnywhere, BlueprintReadWrite, Category="Sequence Perf Test", meta=(AllowedClasses="/Script/Engine.World"))
	FSoftObjectPath Map;
	
	UPROPERTY(Config, EditAnywhere, BlueprintReadWrite, Category="Sequence Perf Test", meta=(AllowedClasses="/Script/LevelSequence.LevelSequence"))
	FSoftObjectPath Sequence;

	/*
	 * The name of the alias of the game mode you can optionally override when opening the level
	 */
	UPROPERTY(Config, EditAnywhere, BlueprintReadWrite, Category="Sequence Perf Test", meta=(AllowedClasses="/Script/GameFramework.GameModeBase"))
	FString GameModeOverride;
};


UCLASS(MinimalAPI, BlueprintType, Config=Engine, DefaultConfig, DisplayName="Automated Performance Testing | Sequence")
class UAutomatedSequencePerfTestProjectSettings : public UDeveloperSettings
{
	GENERATED_BODY()

public:
	UE_API UAutomatedSequencePerfTestProjectSettings(const FObjectInitializer&);

	/** Gets the settings container name for the settings, either Project or Editor */
	virtual FName GetContainerName() const override { return FName("Project"); }
	/** Gets the category for the settings, some high level grouping like, Editor, Engine, Game...etc. */
	virtual FName GetCategoryName() const override { return FName("Plugins"); }

	/*
	 * When the project is run with a Sequence Perf Test, cycle through the input maps, and load and run the associated sequence
	 * outputting separate profiling results for each map/sequence combo
	 * Can be overridden via commandline with -AutomatedPerfTest.SequencePerfTest.TestName, which will only run the test with the matching name
	 */
	UPROPERTY(Config, EditAnywhere, BlueprintReadWrite, Category="Sequence Perf Test")
	TArray<FAutomatedPerfTestMapSequenceCombo> MapsAndSequencesToTest;

	UFUNCTION(BlueprintCallable, Category="Sequence Perf Test")
	UE_API bool GetComboFromTestName(FName TestName, FAutomatedPerfTestMapSequenceCombo& FoundSequence) const;

	/* How long to delay between setting up the sequence for the map before the sequence actually starts */
	UPROPERTY(Config, EditAnywhere, BlueprintReadWrite, Category="Sequence Perf Test")
	float SequenceStartDelay=5.0;
};

/**
 * 
 */
UCLASS(MinimalAPI)
class UAutomatedSequencePerfTest : public UAutomatedPerfTestControllerBase
{
	GENERATED_BODY()

public:
	// ~Begin UAutomatedPerfTestControllerBase Interface
	UE_API virtual FString GetPerfTestTypeID() const override;
	UE_API virtual void SetupTest() override;
	
	UFUNCTION()
	UE_API virtual void RunTest() override;
	
	UE_API virtual void TeardownTest(bool bExitAfterTeardown = true) override;

	UE_API void GatherTestMetadata(TArray<TPair<FString, FString>>& OutMetadata) const override;
	// ~End UAutomatedPerfTestControllerBase Interface
	
	// This function is called on world change to set up the map for the correct map/sequence combo
    UFUNCTION()
    UE_API void NextMap();

	UFUNCTION()
	UE_API void OnSequenceFinished();
	
	UFUNCTION()
	UE_API void OnCameraCut(UCameraComponent* CameraComponent);
	
	UE_API FString GetCameraCutID();
	
protected:
	UE_API virtual void OnInit() override;
	UE_API virtual void UnbindAllDelegates() override;

private:
	const UAutomatedSequencePerfTestProjectSettings* Settings;

	FName SequenceTestName;
	TOptional<FAutomatedPerfTestMapSequenceCombo> CurrentMapSequenceCombo;
	TArray<FAutomatedPerfTestMapSequenceCombo> MapSequenceCombos;

	ALevelSequenceActor* SequenceActor;
	ULevelSequencePlayer* SequencePlayer;

	UCameraComponent* CurrentCamera;
	int NumCameraCuts; // TODO this is temporary until a reliable method is found for getting the user-set name of a camera cut out of Sequencer in packaged builds
};

#undef UE_API
