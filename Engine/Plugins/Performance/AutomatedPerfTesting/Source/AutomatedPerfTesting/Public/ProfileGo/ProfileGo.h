// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Engine/TimerHandle.h"
#include "CoreGlobals.h"
#include "AutomatedPerfTestControllerBase.h"
#include "Subsystems/WorldSubsystem.h"
#include "Stats/Stats.h"

#include "ProfileGo.generated.h"

/*
 * FYI: ProfileGo is WIP and is subject to change.
*/

class FOutputDeviceFile;
class UProfileGoSubsystem;
struct FProfileGoCommandAPT;

#ifndef UE_API
#define UE_API AUTOMATEDPERFTESTING_API
#endif

UENUM()
enum class EProfileGoStateAPT : int8
{
	None,
	StartingRequest,
	SettlingLocation,
	RunningCommands,
	CompletedScenario,
	Summary,
	Completed
};


USTRUCT(BlueprintType, Category = "APT")
struct FProfileGoScenarioAPT
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY(Config, EditAnywhere, BlueprintReadWrite, Category = "APT")
	FString Name;

	UPROPERTY(Config, EditAnywhere, BlueprintReadWrite, Category = "APT")
	FVector Position;

	UPROPERTY(Config, EditAnywhere, BlueprintReadWrite, Category = "APT")
	FRotator Orientation;

	UPROPERTY(Config, EditAnywhere, BlueprintReadWrite, Category = "APT")
	FString OnBegin;

	UPROPERTY(Config, EditAnywhere, BlueprintReadWrite, Category = "APT")
	FString OverrideCommands;

	UPROPERTY(Config, EditAnywhere, BlueprintReadWrite, Category = "APT")
	FString OnEnd;

	UPROPERTY()
	bool AutoGenerated = false;

	UPROPERTY()
	bool SkipRunCommands = false;

	FProfileGoScenarioAPT()
		: Position(ForceInitToZero)
		, Orientation(ForceInitToZero)
	{
		Name = FString();
		OnBegin = FString();
		OverrideCommands = FString();
		OnEnd = FString();
	}
};

USTRUCT(BlueprintType, Category = "APT")
struct FProfileGoGeneratedScenarioAPT : public FProfileGoScenarioAPT
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY()
	FString ScenarioCommand;

	UPROPERTY()
	FString Arguments;

	UPROPERTY()
	FString CopyLocation;

	FProfileGoGeneratedScenarioAPT()
		: FProfileGoScenarioAPT()
	{
		ScenarioCommand = FString();
		Arguments = FString();
		CopyLocation = FString();
	}

	TArray<FProfileGoCommandAPT> Generate() const;
};

USTRUCT(BlueprintType, Category = "APT")
struct FProfileGoCollectionAPT
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY(Config, EditAnywhere, BlueprintReadWrite, Category = "APT")
	FString Name;

	UPROPERTY(Config, EditAnywhere, BlueprintReadWrite, Category = "APT")
	TArray<FString> Scenarios;
};

USTRUCT(BlueprintType, Category = "APT")
struct FProfileGoCommandAPT
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY(Config, EditAnywhere, BlueprintReadWrite, Category = "APT")
	FString Group;

	UPROPERTY(Config, EditAnywhere, BlueprintReadWrite, Category = "APT")
	FString Command;

	UPROPERTY(Config, EditAnywhere, BlueprintReadWrite, Category = "APT")
	float Wait = 0.0f;

	UPROPERTY(Config, EditAnywhere, BlueprintReadWrite, Category = "APT")
	FString Log;

	UPROPERTY(Config, EditAnywhere, BlueprintReadWrite, Category = "APT")
	bool CopyOutputToGameLog = true;

	UPROPERTY(Config, EditAnywhere, BlueprintReadWrite, Category = "APT")
	bool WaitForCallback = false;

	FProfileGoCommandAPT()
	{
	}
};

// Callback for state changes
DECLARE_MULTICAST_DELEGATE(FProfileGoOnRequestFailed);
DECLARE_MULTICAST_DELEGATE(FProfileGoOnPassStarted);
DECLARE_MULTICAST_DELEGATE(FProfileGoOnPassEnded);
DECLARE_MULTICAST_DELEGATE_OneParam(FProfileGoOnScenarioStarted, const FString&);
DECLARE_MULTICAST_DELEGATE_OneParam(FProfileGoOnScenarioEnded, const FString&);


USTRUCT()
struct FProfileGoScenarioResultAPT
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY()
	FProfileGoScenarioAPT Scenario;

	bool bArrivedAtLocation = false;

	bool bRetriedTeleport = false;

	bool bRetracedZ = false;

	bool bCompleted = false;

	bool bRanGC = false;

	float AsyncLoadTime = 0.0f;

	FProfileGoScenarioResultAPT()
	{
		Scenario = FProfileGoScenarioAPT();
	}
};

/* Contains data about the current profilego request */
USTRUCT()
struct FProfileGoRequestAPT
{
	GENERATED_USTRUCT_BODY()

	// profile name passed into the request
	FString RequestName;

	// arguments passed into the request
	FString RequestArgs;

	// a list of scenarios to run
	TArray<FProfileGoScenarioAPT> PendingScenarios;

	// a list of results from each location
	TArray<FProfileGoScenarioResultAPT> ScenarioResults;

	// current result
	UPROPERTY()
	FProfileGoScenarioResultAPT CurrentResult;

	// Commands to be executed (generate anew at each location)
	TArray<FProfileGoCommandAPT> PendingCommands;

	// Time to settle before running commands
	float SettleTime = 0.0f;

	// Do we want to skip settling and everything that comes with it.
	bool bSkipSettling = false;

	// Do we want to ignore whether the player pawn exists and skip all functionality related to it.
	bool bIgnorePawn = false;

	// Use FindBestStandingPoint() during settling to retrace the Z coordinate of each scenario
	bool bRetraceZ = false;

	FProfileGoRequestAPT()
	{
	}
};


/**
 * ProfileGo Automated Perf Test 
 */
UCLASS(Config = Game, MinimalAPI)
class UProfileGo : public UObject
{
	GENERATED_BODY()

public:
	UE_API UProfileGo();

	// This getter is primarily used as a static way to access Config data.
	static UProfileGo& GetCDO()
	{
		UProfileGo* CDO = UProfileGo::StaticClass()->GetDefaultObject<UProfileGo>();
		return *CDO;
	}

	UE_API bool IsRunning() const;
	UE_API void Tick(float DeltaTime);

	UE_API bool Run(UProfileGoSubsystem* InSubsystem, const TCHAR* InProfileName, const TCHAR* InProfileArgs);

	UE_API void AddNewProfileGoCollection(FProfileGoCollectionAPT NewCollection);
	UE_API void AddNewProfileGoScenario(FProfileGoScenarioAPT NewScenario);
	UE_API bool AddScenariosToCurrentRequest(TArray<FProfileGoScenarioAPT> Scenarios, int InsertIndex = INDEX_NONE);

	// Returns true if a scenario or collection was found matching ProfileName. Begins test pass once all scenario names passed through ProfileGo have been added to the CurrentRequest.
	UE_API bool AddScenariosToCurrentRequest(const TCHAR* ProfileName, bool bErrorIfNotFound = false, int InsertIndex = INDEX_NONE);
	UE_API bool TryFindScenarios(const TCHAR* ProfileName, TArray<FProfileGoScenarioAPT>& OutScenarios);
	UE_API bool WaitForLoadingAndStreaming(UWorld* InWorld, float DeltaTime, bool bIncrementalResourceStreaming = false);
	UE_API bool HasEncounteredError() const;

	UE_API FString GetStatus() const;
	UE_API FString GetCurrentScenarioName() const;

	UE_API void LoadFromJSON(const FString& Filename);
	UE_API void SaveToJSON(const FString& Filename);
	UE_API void AddScenariosInLevel(UWorld* World);
	UE_API void UpdateProjectSettings();

	DECLARE_DELEGATE_RetVal_TwoParams(bool, CommandHandlerDelegate, FString&, const FProfileGoCommandAPT& ProfileGoCommand)
	void RegisterCommandDelegate(const FString& Command, CommandHandlerDelegate&& Handler);

	DECLARE_DELEGATE_RetVal_TwoParams(bool, GeneratedScenarioHandlerDelegate, FString&, FString&)
	void RegisterGeneratedScenarioDelegate(const FString& Command, GeneratedScenarioHandlerDelegate&& Handler);

protected:
	UE_API virtual void TickStartingRequest(float DeltaTime);
	UE_API virtual void TickSettling(float DeltaTime);
	UE_API virtual void TickProfiling(float DeltaTime);
	UE_API virtual void TickCompletedScenario(float DeltaTime);
	UE_API virtual void TickSummary(float DeltaTime);
	UE_API virtual void TickCompleted(float DeltaTime);

	UE_API virtual void StartNextScenario();
	UE_API virtual void TeleportToScenario(FProfileGoScenarioAPT& CurrentScenario);
	UE_API virtual bool TickSettlingLocation(float DeltaTime);

	UE_API void AbortWithError(const TCHAR* InErrorMessage);
	UE_API void ConsoleCommand(const TCHAR* InCommand);
	UE_API void ClientMessage(const TCHAR* InMsg);
	UE_API void StartCommandLog(const FProfileGoCommandAPT& ProfileGoCommand);
	UE_API void StopCommandLog();

	UE_API virtual bool TryGenerateScenariosForCurrentRequest(const TCHAR* InProfileName, const TCHAR* InProfileArgs);
	UE_API bool TryParseValue(const TCHAR* Stream, const TCHAR* ParamName, FString& OutValue, bool bShouldStopOnSeparator = false);

	UE_API void ExecuteProfileGoCommand(const FProfileGoCommandAPT& ProfileGoCommand);

	UFUNCTION()
	UE_API void MarkCommandComplete();

	UFUNCTION()
	UE_API void WaitForCommandComplete(float MaxWaitSeconds = 10.f);
	FTimerHandle CommandCompleteFallbackTimerHandle;

	UFUNCTION()
	UE_API void WaitForStreamingCommandComplete();

	UFUNCTION()
	UE_API void CheckIfStreamingCompleted();

	UE_API virtual bool TryHandleProfileGoCommandString(const FProfileGoCommandAPT& ProfileGoCommand);

	UE_API void RegisterInternalCommandDelegates();
	UE_API void RegisterInternalGeneratedScenarioDelegates();

	UE_API APlayerController* GetPlayerController() const;

	UE_API void SetState(EProfileGoStateAPT InState);
	UE_API FString GetStateName(EProfileGoStateAPT InState) const;

	static UE_API bool FindBestStandingPoint(UWorld* InWorld, FVector StartXyHalfHeightZ, FVector& OutPos, bool bIgnoreKillVolume = false, bool bIgnoreBlockingVolume = false);

	UE_API TArray<FProfileGoCommandAPT> CreateCommandList(const FString& InCommandString);
	UE_API virtual bool TryExecuteNextPendingCommand();
	UE_API FProfileGoScenarioAPT CreateScenarioHere(const TCHAR* ScenarioName);

	TArray<FProfileGoScenarioAPT>& GetScenarioTemplates() { return GetCDO().ProfileGoScenarios; }
	TArray<FProfileGoGeneratedScenarioAPT>& GetGeneratedScenarioTemplates() { return GetCDO().ProfileGoGeneratedScenarios; }
	TArray<FProfileGoCollectionAPT>& GetCollectionTemplates() { return GetCDO().ProfileGoCollections; }
	TArray<FProfileGoCommandAPT>& GetCommandTemplates() { return GetCDO().ProfileGoCommands; }

	UE_API bool CollectionExists(const FString& Name);
	UE_API bool ScenarioExists(const FString& Name);

protected:

	// The number of requested scenarios or collections requested by the ProfileGo command. The test pass will not begin until this is decremented to zero by AddScenariosToCurrentRequest()
	int NumRequestedScenarios;
	TSharedPtr<FOutputDeviceFile> ResultsFile;
	TSet<FString> PreviousLogs;

	UPROPERTY(Config)
	float DefaultSettleTime;

	DECLARE_DELEGATE_OneParam(StateTickerDelegate, float)
	TMap<EProfileGoStateAPT, StateTickerDelegate> StateTickers;

	TMap<FString, CommandHandlerDelegate>			CommandHandlers;
	TMap<FString, GeneratedScenarioHandlerDelegate> GenerateScenarioHandlers;

	EProfileGoStateAPT					CurrentState;
	TSharedPtr<FProfileGoRequestAPT>	CurrentRequest;

	bool	bEncounteredError;
	FString ErrorMessage;
	bool	bFirstTickInState;
	float	TimeInState;
	double	LastPlatformTimeSeconds;
	bool	bWaitingOnCommandComplete;
	double	LastTeleportPlatformTimeSeconds;
	int32	ResourceStreamingIteration;

	int32	DefaultSyncInterval;
	int32	DutyCycle;
	double	SenarioStartTime;
	float	DutyCycleTime;

	UProfileGoSubsystem* ProfileGoSubsystem;

private:
	UPROPERTY(Config)
	TArray<FProfileGoScenarioAPT> ProfileGoScenarios;

	UPROPERTY(Config)
	TArray<FProfileGoGeneratedScenarioAPT> ProfileGoGeneratedScenarios;

	UPROPERTY(Config)
	TArray<FProfileGoCollectionAPT> ProfileGoCollections;

	UPROPERTY(Config)
	TArray<FProfileGoCommandAPT> ProfileGoCommands;
};

#undef UE_API