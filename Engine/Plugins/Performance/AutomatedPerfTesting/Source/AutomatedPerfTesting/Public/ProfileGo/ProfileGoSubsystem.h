// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "CoreGlobals.h"

#include "Engine/TimerHandle.h"
#include "Subsystems/WorldSubsystem.h"
#include "Stats/Stats.h"
#include "ProfileGo/ProfileGo.h"

#include "ProfileGoSubsystem.generated.h"

#ifndef UE_API
#define UE_API AUTOMATEDPERFTESTING_API
#endif

class UAutomatedPerfTestControllerBase;

UCLASS(MinimalAPI)
class UProfileGoSubsystem : public UTickableWorldSubsystem
{
	GENERATED_BODY()
public:

	// USubsystem implementation Begin
	UE_API virtual void Initialize(FSubsystemCollectionBase& Collection) override;
	UE_API virtual void Deinitialize() override;
	virtual TStatId GetStatId() const override { RETURN_QUICK_DECLARE_CYCLE_STAT(UProfileGoSubsystem, STATGROUP_Tickables); };
	// USubsystem implementation End

	// FTickableGameObject implementation Begin
	UE_API virtual bool IsTickable() const override;
	UE_API virtual void Tick(float DeltaTime) override;
	// FTickableGameObject implementation End

	template <typename TProfileGoClass>
	bool Run(const TCHAR* ProfileName, const TCHAR* ProfileArgs) { return Run(TProfileGoClass::StaticClass(), ProfileName, ProfileArgs); }
	UE_API bool Run(TSubclassOf<UProfileGo> ProfileGoClass, const TCHAR* ProfileName, const TCHAR* ProfileArgs);

	FString GetStatusMessage() const { return ProfileGo ? ProfileGo->GetStatus() : FString(); }
	bool IsRunning() const { return ProfileGo ? ProfileGo->IsRunning() : false; }
	bool HasEncounteredError() const { return ProfileGo ? ProfileGo->HasEncounteredError() : false; }
	static bool WaitForLoadingAndStreaming(UWorld* InWorld, float DeltaTime, bool bIncrementalResourceStreaming = false) { return UProfileGo::GetCDO().WaitForLoadingAndStreaming(InWorld, DeltaTime, bIncrementalResourceStreaming); }

	FProfileGoOnRequestFailed& OnRequestFailed() { return RequestFailedDelegate; }
	FProfileGoOnScenarioStarted& OnScenarioStarted() { return ScenarioStartedDelegate; }
	FProfileGoOnScenarioEnded& OnScenarioEnded() { return ScenarioEndedDelegate; }
	FProfileGoOnPassStarted& OnPassStarted() { return PassStartedDelegate; }
	FProfileGoOnPassEnded& OnPassEnded() { return PassEndedDelegate; }

	void LoadFromJSON(const FString& Filename) { return UProfileGo::GetCDO().LoadFromJSON(Filename); }
	void SaveToJSON(const FString& Filename) { return UProfileGo::GetCDO().SaveToJSON(Filename); }

	void SetTestController(UAutomatedPerfTestControllerBase* Controller) { TestController = Controller; };
	UAutomatedPerfTestControllerBase* GetTestController() const { return TestController.Get(); }
protected:

	UPROPERTY()
	TObjectPtr<UProfileGo> ProfileGo;

	FProfileGoOnRequestFailed RequestFailedDelegate;
	FProfileGoOnScenarioStarted ScenarioStartedDelegate;
	FProfileGoOnScenarioStarted ScenarioEndedDelegate;
	FProfileGoOnPassStarted PassStartedDelegate;
	FProfileGoOnPassStarted PassEndedDelegate;

	TObjectPtr<UAutomatedPerfTestControllerBase> TestController;
};

#undef UE_API
