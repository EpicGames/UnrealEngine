// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Camera/CameraActor.h"
#include "AutomatedPerfTestStaticCamera.generated.h"

UCLASS(MinimalAPI, Blueprintable, BlueprintType, ClassGroup=Performance)
class AAutomatedPerfTestStaticCamera : public ACameraActor
{
	GENERATED_BODY()

public:
	AAutomatedPerfTestStaticCamera(const FObjectInitializer& ObjectInitializer);

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "ProfileGo | Scenario", meta = (DefaultValue = "Default"))
	FString CollectionName;
};
