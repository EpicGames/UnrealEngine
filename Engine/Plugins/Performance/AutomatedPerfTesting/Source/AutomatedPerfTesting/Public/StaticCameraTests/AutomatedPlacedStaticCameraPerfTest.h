// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "StaticCameraTests/AutomatedStaticCameraPerfTestBase.h"
#include "UObject/Object.h"
#include "Templates/SubclassOf.h"
#include "Engine/EngineTypes.h"
#include "AutomatedPlacedStaticCameraPerfTest.generated.h"

#define UE_API AUTOMATEDPERFTESTING_API

UCLASS(MinimalAPI)
class UAutomatedPlacedStaticCameraPerfTest : public UAutomatedStaticCameraPerfTestBase
{
	GENERATED_BODY()

public:
	UE_API virtual TArray<ACameraActor*> GetMapCameraActors() override;
};

#undef UE_API
