// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "AutomatedPerfTestControllerBase.h"
#include "UObject/Object.h"
#include "Engine/DeveloperSettings.h"
#include "AutomatedStaticCameraPerfTestBase.generated.h"

#define UE_API AUTOMATEDPERFTESTING_API


UCLASS(MinimalAPI, BlueprintType, Config=Engine, DefaultConfig, DisplayName="Automated Performance Testing | Static Camera")
class UAutomatedStaticCameraPerfTestProjectSettings : public UDeveloperSettings
{
	GENERATED_BODY()
	
public:
	UE_API UAutomatedStaticCameraPerfTestProjectSettings(const FObjectInitializer&);

	/** Gets the settings container name for the settings, either Project or Editor */
	virtual FName GetContainerName() const override { return FName("Project"); }
	/** Gets the category for the settings, some high level grouping like, Editor, Engine, Game...etc. */
	virtual FName GetCategoryName() const override { return FName("Plugins"); }

	UFUNCTION(BlueprintCallable, Category="Static Camera Perf Test")
	UE_API bool GetMapFromAssetName(FString AssetName, FSoftObjectPath& OutSoftObjectPath) const;
	
	/*
	 * List of levels to test
	 */
	UPROPERTY(Config, EditAnywhere, BlueprintReadWrite, Category="Static Cameras", meta=(AllowedClasses="/Script/Engine.World"))
	TArray<FSoftObjectPath> MapsToTest;

	/*
     * If set, will launch the material performance test map with this game mode alias (make sure you've set the game mode alias in
     * the Maps and Modes settings of your project!)
     */
	UPROPERTY(Config, EditAnywhere, BlueprintReadWrite, Category="Static Cameras")
	FString GameModeOverride;
	
	/*
	 * If true, will capture a screenshot for each camera tested after gathering data
	 */
	UPROPERTY(Config, EditAnywhere, BlueprintReadWrite, Category="Static Cameras")
	bool bCaptureScreenshots;

	/*
	 * For how long the material performance test should delay before beginning to gather data for a material, in seconds
     */
	UPROPERTY(Config, EditAnywhere, BlueprintReadWrite, Category="Static Cameras")
	float WarmUpTime;
	
	/*
	 * For how long the static camera performance test should gather data on each camera, in seconds
	 */
	UPROPERTY(Config, EditAnywhere, BlueprintReadWrite, Category="Static Cameras")
	float SoakTime;

	/*
	 * For how long the static camera performance test should delay after ending evaluation before switching to the next camera
     */
	UPROPERTY(Config, EditAnywhere, BlueprintReadWrite, Category="Static Cameras")
	float CooldownTime;
};

/**
 * 
 */
UCLASS(MinimalAPI)
class UAutomatedStaticCameraPerfTestBase : public UAutomatedPerfTestControllerBase
{
	GENERATED_BODY()
	
public:
	// ~Begin UAutomatedPerfTestControllerBase Interface
	UE_API virtual void SetupTest() override;
	
	UFUNCTION()
	UE_API virtual void RunTest() override;

	UE_API virtual FString GetPerfTestTypeID() const override;

	UE_API virtual void GatherTestMetadata(TArray<TPair<FString, FString>>& OutMetadata) const override;
	// ~End UAutomatedPerfTestControllerBase Interface
	
	UFUNCTION()
	UE_API void SetUpNextCamera();

	UFUNCTION()
	UE_API void EvaluateCamera();
	
	UFUNCTION()
	UE_API void FinishCamera();
	
	UFUNCTION()
	UE_API void ScreenshotCamera();

	UFUNCTION()
	UE_API void NextMap();

	UE_API virtual TArray<ACameraActor*> GetMapCameraActors();
	
	UE_API ACameraActor* GetCurrentCamera() const;
	UE_API FString GetCurrentCameraRegionName() const;

	UE_API void MarkCameraStart();
	UE_API void MarkCameraEnd();
	
protected:
	UE_API virtual void OnInit() override;
	UE_API virtual void UnbindAllDelegates() override;
	UE_API virtual void TriggerExitAfterDelay() override;

private:
	UPROPERTY()
	TArray<TObjectPtr<ACameraActor>> CamerasToTest;

	UPROPERTY()
	TObjectPtr<ACameraActor> CurrentCamera;
	
	FString CurrentMapName;
	FSoftObjectPath CurrentMapPath;
	TArray<FSoftObjectPath> MapsToTest;
	const UAutomatedStaticCameraPerfTestProjectSettings* Settings;
};

#undef UE_API
